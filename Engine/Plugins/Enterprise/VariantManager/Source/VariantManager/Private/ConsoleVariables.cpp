// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariables.h"

namespace UE::VariantManager
{
	TAutoConsoleVariable<int32> CVar_VariantManager_CapturePropertyDialog(
		TEXT("VariantManager.CapturePropertyDialog"),
		0,
		TEXT("Controls which Capture Property Dialog UI is used (0=default, 1=updated with last used, two columns)"));

	TAutoConsoleVariable<bool> CVar_VariantManager_FunctionArgumentsUI(
		TEXT("VariantManager.FunctionArgumentsUI"),
		false,
		TEXT("Enables UI for editing function caller arguments"));

	TAutoConsoleVariable<bool> CVar_VariantManager_AssetPicker(
		TEXT("VariantManager.AssetPicker"),
		false,
		TEXT("Improves the layout and adds asset picker to switch between Level Variant Sets."));


	// Controls which Capture Property Dialog UI is used (0=default, 1=updated with last used, two columns)
	TAutoConsoleVariable<int32>& Get_CVar_VariantManager_CapturePropertyDialog() { return CVar_VariantManager_CapturePropertyDialog; }
	// Enables UI for editing function caller arguments
	TAutoConsoleVariable<bool>& Get_CVar_VariantManager_FunctionArgumentsUI() { return CVar_VariantManager_FunctionArgumentsUI; }
	// Improves the layout and adds asset picker to switch between Level Variant Sets.
	TAutoConsoleVariable<bool>& Get_CVar_VariantManager_AssetPicker() { return CVar_VariantManager_AssetPicker; }

}