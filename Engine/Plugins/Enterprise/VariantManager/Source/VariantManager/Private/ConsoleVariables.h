// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/ConsoleManager.h"

namespace UE::VariantManager
{
	// Controls which Capture Property Dialog UI is used (0=default, 1=updated with last used, two columns)
	TAutoConsoleVariable<int32>& Get_CVar_VariantManager_CapturePropertyDialog();
	// Enables UI for editing function caller arguments
	TAutoConsoleVariable<bool>& Get_CVar_VariantManager_FunctionArgumentsUI();
	// Improves the layout and adds asset picker to switch between Level Variant Sets.
	TAutoConsoleVariable<bool>& Get_CVar_VariantManager_AssetPicker();
}

