// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCaptureDialog.h"

#include "ConsoleVariables.h"
#include "Editor.h"
#include "SCapturedActorsWidgetTwo.h"
#include "SCapturedPropertiesWidgetTwo.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Dialogs/SCapturedPropertiesWidget.h"
#include "Dialogs/SCapturedActorsWidget.h"
#include "Misc/ConfigCacheIni.h"
#include "VariantManagerLog.h"
#include "VariantManagerPropertyCapturer.h"
#include "GameFramework/GameUserSettings.h"
#include "Application/SlateApplicationBase.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SCaptureDialog"
#define ACTOR_PROPERTY_DEFAULT_WIDTH 500.0f
#define ACTOR_PROPERTY_DEFAULT_HEIGHT 730.0f
#define PROPERTY_DEFAULT_WIDTH 500.0f
#define PROPERTY_DEFAULT_HEIGHT 571.0f


void LoadWidthHeightFromIniFile(float& OutWidth, float& OutHeight, FString WindowName)
{
	FString WidthKey = *(WindowName + TEXT("Width"));  // Something like 'ActorPropertyDialogWidth'
	FString HeightKey = *(WindowName + TEXT("Height"));  // Something like 'ActorPropertyDialogHeight'

	// Try loading width
	FString WidthStr;
	if (GConfig->GetString(TEXT("VariantManager"), *WidthKey, WidthStr, GEditorPerProjectIni))
	{
		OutWidth = FCString::Atof(*WidthStr);
	}

	// Try loading height
	FString HeightStr;
	if (GConfig->GetString(TEXT("VariantManager"), *HeightKey, HeightStr, GEditorPerProjectIni))
	{
		OutHeight = FCString::Atof(*HeightStr);
	}
}

void SaveWidthHeightToIniFile(float InWidth, float InHeight, FString WindowName)
{
	GConfig->SetString(TEXT("VariantManager"), *(WindowName + TEXT("Width")), *FString::SanitizeFloat(InWidth), GEditorPerProjectIni);
	GConfig->SetString(TEXT("VariantManager"), *(WindowName + TEXT("Height")), *FString::SanitizeFloat(InHeight), GEditorPerProjectIni);
}

void SanitizeWindowSize(float& InOutWidth, float& InOutHeight, const TSharedRef<SWindow> Window)
{
	FIntPoint Res = GEngine->GetGameUserSettings()->GetDesktopResolution();

	float ScaleFactor = 1.0f;
	FVector2D BorderSize = FVector2D::ZeroVector;

	// This might be called on load, where Window hasn't been properly created yet. It will return
	// 1.0f for DPIScaleFactor every time in that condition
	if (Window->IsActive())
	{
		FVector2D ClientSize = Window->GetClientSizeInScreen();
		FVector2D WindowSize = Window->GetWindowSizeFromClientSize(ClientSize);
		BorderSize = WindowSize - ClientSize;

		ScaleFactor = Window->GetDPIScaleFactor() * FSlateApplicationBase::Get().GetApplicationScale();
	}
	// Check the top level window. This can either be the VariantManager tab itself or the editor window,
	// depending on where the last action started (e.g. a drag from the scene outliner to the variant manager
	// will count as the editor, but right-clicking the variant manager will count for the variant manager)
	else
	{
		TSharedPtr<SWindow> TopLevelWindow = FSlateApplicationBase::Get().GetActiveTopLevelWindow();
		if (TopLevelWindow.IsValid())
		{
			FVector2D ClientSize = TopLevelWindow->GetClientSizeInScreen();
			FVector2D WindowSize = TopLevelWindow->GetWindowSizeFromClientSize(ClientSize);
			BorderSize = WindowSize - ClientSize;

			ScaleFactor = TopLevelWindow->GetDPIScaleFactor() * FSlateApplicationBase::Get().GetApplicationScale();
		}
	}

	// Usual borders at 100% scaling
	BorderSize.X = FMath::Max(BorderSize.X, 10.0f);
	BorderSize.Y = FMath::Max(BorderSize.Y, 34.0f);

	if (ScaleFactor <= 0.0f)
	{
		ScaleFactor = 1.0f;
	}

	// Some min values or else it can literally be 1 or 2 pixels wide
	InOutWidth = FMath::Clamp(InOutWidth, 136.0f, ((float)Res.X - BorderSize.X) / ScaleFactor);
	InOutHeight = FMath::Clamp(InOutHeight, 40.0f, ((float)Res.Y - BorderSize.Y) / ScaleFactor);
}

void SCaptureDialog::Construct(const FArguments& InArgs)
{
	bUserAccepted = false;
	ECaptureDialogType DialogType = InArgs._DialogType;
	const TArray<UObject*>* ObjectsToCapture = InArgs._ObjectsToCapture;
	if (ObjectsToCapture)
	{
		FVariantManagerPropertyCapturer::CaptureProperties(*ObjectsToCapture, CapturedProperties);
	}
	else
	{	// No objects to display, default to property only dialog to avoid errors
		DialogType = ECaptureDialogType::Property;
	}

	FString WindowName;
	FString WindowTitle;
	float InitialWidth;
	float InitialHeight;

	switch (DialogType)
	{
	case ECaptureDialogType::Property:
		InitialWidth = PROPERTY_DEFAULT_WIDTH;
		InitialHeight = PROPERTY_DEFAULT_HEIGHT;
		WindowName = TEXT("PropertyDialog");
		WindowTitle = TEXT("Capture new properties");
		break;
	case ECaptureDialogType::ActorAndProperty:
		InitialWidth = ACTOR_PROPERTY_DEFAULT_WIDTH;
		InitialHeight = ACTOR_PROPERTY_DEFAULT_HEIGHT;
		WindowName = TEXT("ActorPropertyDialog");
		WindowTitle = TEXT("Capture new actors and properties");
		break;
	default:
		UE_LOGF(LogVariantManager, Error, "Invalid SCaptureDialog type!");
		return;
	}
	
	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	DialogVersion = UE::VariantManager::Get_CVar_VariantManager_CapturePropertyDialog().GetValueOnAnyThread();

	if (DialogVersion == 1)
	{
		Splitter = SNew(SSplitter)
			.Orientation(Orient_Vertical);

		if (DialogType == ECaptureDialogType::ActorAndProperty)
		{
			ActorWidget = SNew(SCapturedActorsWidgetTwo).Actors(ObjectsToCapture);
			Splitter->AddSlot()
			        .SizeRule(SSplitter::FractionOfParent)
			        .Value(0.2)
			[
				SNew(SBox)
				.Padding(0.0f, 0.0f, 0.0f, 10.0f)
				[
					ActorWidget.ToSharedRef()
				]
			];
		}

		PropertiesWidget = SNew(SCapturedPropertiesWidgetTwo).PropertyPaths(&CapturedProperties);
		SearchBox = SNew(SSearchBox)
			.HintText(LOCTEXT("SCapturedPropertiesWidget", "Filter"))
			.OnTextChanged(PropertiesWidget.ToSharedRef(), &SCapturedPropertiesWidgetBase::FilterPropertyPaths);

		Splitter->AddSlot()
		        .SizeRule(SSplitter::FractionOfParent)
		        .Value(0.7)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.MaxHeight(20.0f)
			.AutoHeight()
			.VAlign(VAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SearchBox.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 10.0f)
			.AutoHeight()
			.FillHeight(1.0f)
			.VAlign(VAlign_Top)
			[
				PropertiesWidget.ToSharedRef()
			]
		];

		MainVerticalBox->AddSlot()
		               .Padding(0.0f, 0.0f, 0.0f, 10.0f)
		               .FillHeight(1.0f)
		[
			Splitter.ToSharedRef()
		];
	}
	else
	{
		if (DialogType == ECaptureDialogType::ActorAndProperty)
		{
			ActorWidget = SNew(SCapturedActorsWidget).Actors(ObjectsToCapture);
			MainVerticalBox->AddSlot()
			               .Padding(0.0f, 0.0f, 0.0f, 10.0f)
			               .AutoHeight()
			[
				ActorWidget.ToSharedRef()
			];
		}

		PropertiesWidget = SNew(SCapturedPropertiesWidget).PropertyPaths(&CapturedProperties);

		SearchBox = SNew(SSearchBox)
			.HintText(LOCTEXT("SCapturedPropertiesWidget", "Filter"))
			.OnTextChanged(PropertiesWidget.ToSharedRef(), &SCapturedPropertiesWidgetBase::FilterPropertyPaths);

		// This works but conflicts with Enter/Esc to quit/accept
		//SetWidgetToFocusOnActivate(SearchBox);

		MainVerticalBox->AddSlot()
		               .MaxHeight(20.0f)
		               .AutoHeight()
		               .VAlign(VAlign_Fill)
		               .Padding(0.0f, 0.0f, 0.0f, 4.0f)
		[
			SearchBox.ToSharedRef()
		];

		MainVerticalBox->AddSlot()
		               .Padding(0.0f, 0.0f, 0.0f, 10.0f)
		               .AutoHeight()
		               .FillHeight(1.0f)
		               .VAlign(VAlign_Top)
		[
			PropertiesWidget.ToSharedRef()
		];
	}

	MainVerticalBox->AddSlot()
	               .MaxHeight(30.0f)
	               .AutoHeight()
	               .VAlign(VAlign_Center)
	               .HAlign(HAlign_Fill)
	               .Padding(0.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text_Lambda([this]()
				{
					if (ActorWidget && PropertiesWidget)
					{
						return FText::FormatOrdered(LOCTEXT("ActorsPropertiesFmt", "{0} Actors selected and {1} Properties selected"),
							ActorWidget->GetCurrentCheckedActors().Num(),
							PropertiesWidget->GetCurrentCheckedProperties().Num());
					}
					else if (PropertiesWidget)
					{
						return FText::FormatOrdered(LOCTEXT("PropertiesFmt", "{0} Properties selected"),
							PropertiesWidget->GetCurrentCheckedProperties().Num());
					}					

					return FText::GetEmpty();
				})
			.Visibility(UE::VariantManager::Get_CVar_VariantManager_CapturePropertyDialog().GetValueOnAnyThread() == 0 ? EVisibility::Hidden : EVisibility::Visible)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			+SUniformGridPanel::Slot(0,0)
			[
				SNew(SPrimaryButton)
				.Text(NSLOCTEXT("SCapturedPropertiesWidget", "ClassPickerSelectButton", "Select"))
				.OnClicked(this, &SCaptureDialog::OnDialogConfirmed)
			]
			+SUniformGridPanel::Slot(1,0)
			[
				SNew(SButton)
				.Text(NSLOCTEXT("SCapturedPropertiesWidget", "ClassPickerCancelButton", "Cancel"))
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.OnClicked(this, &SCaptureDialog::OnDialogCanceled)
			]
		]
	];

	SetOnWindowClosed(FOnWindowClosed::CreateLambda([=](const TSharedRef<SWindow>& WindowArg)
	{
		FVector2D ClientSize = WindowArg->GetClientSizeInScreen();

		// Remove the effects of DPI scaling before saving
		float ScaleFactor = WindowArg->GetDPIScaleFactor() * FSlateApplicationBase::Get().GetApplicationScale();
		if (ScaleFactor <= 0.0f)
		{
			ScaleFactor = 1.0f;
		}
		float Width = ClientSize.X / ScaleFactor;
		float Height = ClientSize.Y / ScaleFactor;

		SanitizeWindowSize(Width, Height, WindowArg);
		SaveWidthHeightToIniFile(Width, Height, WindowName);
	}));

	LoadWidthHeightFromIniFile(InitialWidth, InitialHeight, WindowName);
	SanitizeWindowSize(InitialWidth, InitialHeight, SharedThis(this));

	SWindow::Construct(SWindow::FArguments()
	.Title(FText::FromString(WindowTitle))
	.AutoCenter(EAutoCenter::PreferredWorkArea)
	.SizingRule(ESizingRule::UserSized)
	.ClientSize(FVector2D(InitialWidth, InitialHeight))
	.IsTopmostWindow(true)
	[
		SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		.Padding(FMargin(10.0f, 10.0f, 10.0f, 10.0f))
		[
			MainVerticalBox
		]
	]);
}

TArray<UObject*> SCaptureDialog::GetCurrentCheckedActors()
{
	if (ActorWidget.IsValid())
	{
		return ActorWidget->GetCurrentCheckedActors();
	}

	return TArray<UObject*>();
}

TArray<TSharedPtr<FCapturableProperty>> SCaptureDialog::GetCurrentCheckedProperties()
{
	if (PropertiesWidget.IsValid())
	{
		return PropertiesWidget->GetCurrentCheckedProperties();
	}

	return TArray<TSharedPtr<FCapturableProperty>>();
}

TSharedPtr<SCaptureDialog> SCaptureDialog::OpenCaptureDialogAsModalWindow(ECaptureDialogType DialogType, const TArray<UObject*>& ObjectsToCapture)
{
	TSharedPtr<SCaptureDialog> CaptureWindow = SNew(SCaptureDialog)
		.DialogType(DialogType)
		.ObjectsToCapture(&ObjectsToCapture);

	GEditor->EditorAddModalWindow(CaptureWindow.ToSharedRef());

	return CaptureWindow;
}

FReply SCaptureDialog::OnDialogConfirmed()
{
	bUserAccepted = true;

	if (DialogVersion == 1 &&
		PropertiesWidget.IsValid())
	{
		StaticCastSharedPtr<SCapturedPropertiesWidgetTwo>(PropertiesWidget)->SaveFavorites();
	}

	RequestDestroyWindow();

	return FReply::Handled();
}

FReply SCaptureDialog::OnDialogCanceled()
{
	bUserAccepted = false;
	RequestDestroyWindow();

	return FReply::Handled();
}

FReply SCaptureDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnDialogCanceled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		return OnDialogConfirmed();
	}

	return FReply::Unhandled();
}


#undef LOCTEXT_NAMESPACE
