// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Layout/SSplitter.h"

class SCapturedPropertiesWidgetBase;
class SCapturedActorsWidget;
class SSearchBox;
struct FCapturableProperty;


enum class ECaptureDialogType : uint8
{
	Property,
	ActorAndProperty
};

class SCaptureDialog : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SCaptureDialog){}
	SLATE_ARGUMENT(ECaptureDialogType, DialogType)
	SLATE_ARGUMENT(const TArray<UObject*>*, ObjectsToCapture)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TArray<UObject*> GetCurrentCheckedActors();
	TArray<TSharedPtr<FCapturableProperty>> GetCurrentCheckedProperties();
	bool GetUserAccepted(){ return bUserAccepted; }

	static TSharedPtr<SCaptureDialog> OpenCaptureDialogAsModalWindow(ECaptureDialogType DialogType, const TArray<UObject*>& ObjectsToCapture);

private:

	FReply OnDialogConfirmed();
	FReply OnDialogCanceled();

	// Overloaded from SWidget
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	TSharedPtr<SCapturedActorsWidget> ActorWidget;
	TSharedPtr<SCapturedPropertiesWidgetBase> PropertiesWidget;
	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<SSplitter> Splitter;

	TArray<TSharedPtr<FCapturableProperty>> CapturedProperties;
	bool bUserAccepted = false;

	TWeakPtr<SWindow> WeakWindow;
	int DialogVersion = 0;
};
