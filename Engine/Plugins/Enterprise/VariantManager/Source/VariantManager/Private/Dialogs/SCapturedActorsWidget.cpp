// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCapturedActorsWidget.h"

#include "Framework/Views/TableViewMetadata.h"
#include "GameFramework/Actor.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCapturedActorsWidget"


void SCapturedActorsWidget::Construct(const FArguments& InArgs)
{
	if (InArgs._Actors)
	{
		for (UObject* Actor : *InArgs._Actors)
		{
			if (Actor)
			{
				ActorChecked.Add(Actor, true);
				AllActors.Add(Actor);
			}
		}
	}

    ChildSlot
	[
		SNew(SExpandableArea)
		.InitiallyCollapsed(false)
		.Padding(0.0f)
		.MaxHeight(150.0f)
		.AreaTitle(NSLOCTEXT("SCapturedActorsWidget", "SelectedActorsText", "Captured Actors"))
		.BodyContent()
		[
			SNew(SListView<UObject*>)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&AllActors)
			.OnGenerateRow(this, &SCapturedActorsWidget::MakeRow)
			.Visibility(EVisibility::Visible)
		]
	];
}

TSharedRef<ITableRow> SCapturedActorsWidget::MakeRow(UObject* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	ensure(ActorChecked.Contains(Item));

	AActor* ItemAsActor = Cast<AActor>(Item);
	if (ensure(ItemAsActor))
	{
		return SNew(STableRow<UObject*>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(8.0f, 2.0f, 10.0f, 4.0f)
			.MaxWidth(15.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("SCapturedActorCheckboxTooltip", "Capture this actor to the variant"))
				.IsChecked(ActorChecked[Item] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged_Lambda([this, Item](ECheckBoxState NewAutoCloseState)
				{
					ActorChecked[Item] = (NewAutoCloseState == ECheckBoxState::Checked);
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f, 2.0f, 4.0f)
			.FillWidth(1.0)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ItemAsActor->GetActorLabel()))
			]
		];
	}

	return SNew(STableRow<UObject*>, OwnerTable);
}

TArray<UObject*> SCapturedActorsWidget::GetCurrentCheckedActors()
{
	TArray<UObject*> CheckedActors;
	for (auto Pair : ActorChecked)
	{
		if (Pair.Value)
		{
			CheckedActors.Add(Pair.Key);
		}
	}

	return CheckedActors;
}

#undef LOCTEXT_NAMESPACE
