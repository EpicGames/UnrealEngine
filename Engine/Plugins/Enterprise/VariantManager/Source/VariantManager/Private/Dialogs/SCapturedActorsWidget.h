// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
struct FCapturableProperty;

class SCapturedActorsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCapturedActorsWidget) {}
	SLATE_ARGUMENT(const TArray<UObject*>*, Actors)
	SLATE_END_ARGS()

	SCapturedActorsWidget()
	{
	}

	~SCapturedActorsWidget() = default;

	void Construct(const FArguments& InArgs);
	TArray<UObject*> GetCurrentCheckedActors();

protected: // Changed from "private:" to allow SCapturedActorsWidgetTwo to inherit

	TSharedRef<ITableRow> MakeRow(UObject* Item, const TSharedRef<STableViewBase>& OwnerTable);

	TArray<UObject*> AllActors;
	TMap<UObject*, bool> ActorChecked;
};
