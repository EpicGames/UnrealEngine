// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCapturedActorsWidgetTwo.h"

#include "ClassIconFinder.h"
#include "SlateOptMacros.h"
#include "Framework/Views/TableViewMetadata.h"
#include "GameFramework/Actor.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SCapturedActorsWidgetTwo"

void SCapturedActorsWidgetTwo::Construct(const FArguments& InArgs)
{
	if (InArgs._Actors)
	{
		for (UObject* Actor : *InArgs._Actors)
		{
			if (Actor)
			{
				ActorChecked.Add(Actor, true);
				AllActors.Add(Actor);
			}
		}
	}

    ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 4.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SelectedActorsText", "Captured Actors"))
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(ActorListView, SListView<UObject*>)
			.SelectionMode(ESelectionMode::Multi)
			.ListItemsSource(&AllActors)
			.OnGenerateRow(this, &SCapturedActorsWidgetTwo::MakeRow)
			.OnSelectionChanged(this, &SCapturedActorsWidgetTwo::OnSelectionChanged)
			.Visibility(EVisibility::Visible)
		]
	];

	TArray<UObject*> ActorsToSelect;
	for (const TPair<UObject*, bool>& Pair : ActorChecked)
	{
		if (Pair.Value)
		{
			ActorsToSelect.Add(Pair.Key);
		}
	}

	if (ActorsToSelect.Num() > 0)
	{
		ActorListView->SetItemSelection(ActorsToSelect, true, ESelectInfo::OnNavigation);
	}
}

TSharedRef<ITableRow> SCapturedActorsWidgetTwo::MakeRow(UObject* Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	if (ensure(ActorChecked.Contains(Item)))
	{
		AActor* ItemAsActor = Cast<AActor>(Item);
		if (ensure(ItemAsActor))
		{
			return SNew(STableRow<UObject*>, OwnerTable)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(8.0f, 2.0f, 10.0f, 4.0f)
				.MaxWidth(15.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FClassIconFinder::FindIconForActor(ItemAsActor))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 2.0f, 2.0f, 4.0f)
				.FillWidth(1.0)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ItemAsActor->GetActorLabel()))
				]
			];
		}
	}

	return SNew(STableRow<UObject*>, OwnerTable);
}

void SCapturedActorsWidgetTwo::OnSelectionChanged(UObject* Object, ESelectInfo::Type SelectType)
{
	for (UObject* Item : ActorListView->GetItems())
	{
		const bool bSelected = ActorListView->IsItemSelected(Item);
		ActorChecked.Emplace(Item, bSelected);
	}
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
