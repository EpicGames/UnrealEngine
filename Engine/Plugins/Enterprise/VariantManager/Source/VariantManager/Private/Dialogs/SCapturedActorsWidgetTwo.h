// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCapturedActorsWidget.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
struct FCapturableProperty;

class SCapturedActorsWidgetTwo : public SCapturedActorsWidget
{
public:
	SLATE_BEGIN_ARGS(SCapturedActorsWidgetTwo) { }
		SLATE_ARGUMENT(const TArray<UObject*>*, Actors)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
private:
	TSharedRef<ITableRow> MakeRow(UObject* Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnSelectionChanged(UObject* Object, ESelectInfo::Type SelectType);

	TSharedPtr<SListView<UObject*>> ActorListView;
};
