// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCapturedPropertiesWidget.h"

#include "CapturableProperty.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCapturedPropertiesWidget"


void SCapturedPropertiesWidget::Construct(const FArguments& InArgs)
{
	CapturedProperties = *InArgs._PropertyPaths;
	FilteredCapturedProperties = CapturedProperties;

    ChildSlot
	[
		SNew(SExpandableArea)
		.InitiallyCollapsed(false)
		.Padding(0.0f)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.AreaTitle(NSLOCTEXT("SCapturedPropertiesWidget", "CapturedPropertiesText", "Captured Properties"))
		.BodyContent()
		[
			SAssignNew(PropListView, SListView<TSharedPtr<FCapturableProperty>>)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&FilteredCapturedProperties)
			.OnGenerateRow(this, &SCapturedPropertiesWidget::MakeCapturedPropertyWidget)
			.Visibility(EVisibility::Visible)
		]
	];
}

TSharedRef<ITableRow> SCapturedPropertiesWidget::MakeCapturedPropertyWidget(TSharedPtr<FCapturableProperty> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
	SNew(STableRow<TSharedPtr<FPropertyPath>>, OwnerTable)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 2.0f, 10.0f, 4.0f)
		.MaxWidth(15.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("SCapturedPropertyCheckboxTooltip", "Capture this property to the variant"))
			.IsChecked(Item->Checked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged_Lambda([Item](ECheckBoxState NewAutoCloseState)
			{
				Item->Checked = (NewAutoCloseState == ECheckBoxState::Checked);
			})
		]
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 2.0f, 2.0f, 4.0f)
		.FillWidth(1.0)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(FText::FromString(Item->DisplayName))
		]
	];
}

TArray<TSharedPtr<FCapturableProperty>> SCapturedPropertiesWidget::GetCurrentCheckedProperties()
{
	TArray<TSharedPtr<FCapturableProperty>> Result = CapturedProperties;
	Result.RemoveAll([](const TSharedPtr<FCapturableProperty>& PropCapture)
	{
		return !PropCapture->Checked;
	});

	return Result;
}

void SCapturedPropertiesWidget::FilterPropertyPaths(const FText& Filter)
{
	FilteredCapturedProperties.SetNumUninitialized(0, EAllowShrinking::No);

	// Build a list of strings that must be matched
	TArray<FString> FilterStrings;

	FString FilterString = Filter.ToString();
	FilterString.TrimStartAndEndInline();
	FilterString.ParseIntoArray(FilterStrings, TEXT(" "), true /*bCullEmpty*/);

	for (TSharedPtr<FCapturableProperty>& PropCapture : CapturedProperties)
	{
		bool bPassedTextFilter = true;

		// check each string in the filter strings list against
		for (const FString& String : FilterStrings)
		{
			if (!PropCapture->DisplayName.Contains(String))
			{
				bPassedTextFilter = false;
				break;
			}
		}

		if (bPassedTextFilter)
		{
			FilteredCapturedProperties.Add(PropCapture);
		}
	}

	PropListView->RebuildList();
}

#undef LOCTEXT_NAMESPACE
