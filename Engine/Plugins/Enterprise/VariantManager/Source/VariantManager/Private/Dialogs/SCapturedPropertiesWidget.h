// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

class ITableRow;
class STableViewBase;
struct FCapturableProperty;

class SCapturedPropertiesWidgetBase : public SCompoundWidget
{
public:
	virtual TArray<TSharedPtr<FCapturableProperty>> GetCurrentCheckedProperties() = 0;
	virtual void FilterPropertyPaths(const FText& Filter) = 0;
};

class SCapturedPropertiesWidget : public SCapturedPropertiesWidgetBase
{
public:
	SLATE_BEGIN_ARGS(SCapturedPropertiesWidget) {}
	SLATE_ARGUMENT(TArray<TSharedPtr<FCapturableProperty>>*, PropertyPaths)
	SLATE_END_ARGS()

	SCapturedPropertiesWidget()
	{
	}

	~SCapturedPropertiesWidget() = default;

	void Construct(const FArguments& InArgs);
	virtual TArray<TSharedPtr<FCapturableProperty>> GetCurrentCheckedProperties() override;
	virtual void FilterPropertyPaths(const FText& Filter) override;

private:

	TSharedRef<ITableRow> MakeCapturedPropertyWidget(TSharedPtr<FCapturableProperty> Item, const TSharedRef<STableViewBase>& OwnerTable);

	TArray<TSharedPtr<FCapturableProperty>> CapturedProperties;

	TSharedPtr<SListView<TSharedPtr<FCapturableProperty>>> PropListView;
	TArray<TSharedPtr<FCapturableProperty>> FilteredCapturedProperties;
};
