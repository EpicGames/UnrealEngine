// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCapturedPropertiesWidgetTwo.h"

#include "SlateOptMacros.h"

#include "CapturableProperty.h"
#include "VariantManagerStyle.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SCapturedPropertiesWidgetTwo"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void LoadFavoritesFromIniFile(TSet<FName>& OutFavoriteProperties)
{
	OutFavoriteProperties.Empty();

	TArray<FString> Properties;
	GConfig->GetSingleLineArray(TEXT("VariantManager"), TEXT("FavoriteProperties"), Properties, GEditorPerProjectIni);

	Algo::Transform(Properties, OutFavoriteProperties, [](const FString& Value) { return FName(*Value); });
}

void SaveFavoritesToIniFile(const TSet<FName>& FavoriteProperties)
{
	TArray<FString> Properties;
	Algo::Transform(FavoriteProperties, Properties, [](const FName& Name) { return Name.ToString(); });

	GConfig->SetSingleLineArray(TEXT("VariantManager"), TEXT("FavoriteProperties"), Properties, GEditorPerProjectIni);
	GConfig->Flush(false, GEditorPerProjectIni);
}

static FName ColumnFavorite = TEXT("Favorite");
static FName ColumnPath = TEXT("Path");
static FName ColumnProperty = TEXT("Property");

SCapturedPropertiesWidgetTwo::~SCapturedPropertiesWidgetTwo()
{
	SaveFavorites();
}

DECLARE_DELEGATE_OneParam(FFavoriteChanged, bool);

class SFavoriteWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SFavoriteWidget) {}
		SLATE_EVENT(FFavoriteChanged, OnFavoriteChanged)
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedPtr<FCapturableProperty> InTargetRow)
	{
		TargetProperty = InTargetRow;
		OnFavoriteChanged = InArgs._OnFavoriteChanged;

		SImage::Construct(
			SImage::FArguments()
			.ColorAndOpacity(this, &SFavoriteWidget::GetForegroundColor)
			.Image(this, &SFavoriteWidget::GetBrush)
		);

		FavoriteHoveredBrush = FAppStyle::GetBrush("Icons.Star");
		FavoriteNotHoveredBrush = FAppStyle::GetBrush("Icons.Star");
		NotFavoriteHoveredBrush = FAppStyle::GetBrush("PropertyWindow.Favorites_Disabled");
		NotFavoriteNotHoveredBrush = FAppStyle::GetBrush("PropertyWindow.Favorites_Disabled");
	}

private:
	FReply HandleClick() const
	{
		if (!IsEnabled())
		{
			return FReply::Unhandled();
		}

		ToggleFavorite();

		return FReply::Handled();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		return HandleClick();
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return HandleClick();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	const FSlateBrush* GetBrush() const
	{
		if (IsFavorite())
		{
			return IsHovered() ? FavoriteHoveredBrush : FavoriteNotHoveredBrush;
		}
		else
		{
			if (IsEnabled())
			{
				return IsHovered() ? NotFavoriteHoveredBrush : NotFavoriteNotHoveredBrush;
			}
		}

		return FStyleDefaults::GetNoBrush();
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		if (!IsFavorite() && !IsEnabled())
		{
			return FLinearColor::Transparent;
		}

		if (IsHovered())
		{
			return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
		}

		return FSlateColor::UseForeground();
	}

	bool IsFavorite() const
	{
		return TargetProperty ? TargetProperty->Favorite : false;
	}

	void ToggleFavorite() const
	{
		if (TargetProperty)
		{
			OnFavoriteChanged.ExecuteIfBound(!TargetProperty->Favorite);
		}
	}

private:
	TSharedPtr<FCapturableProperty> TargetProperty;
	FFavoriteChanged OnFavoriteChanged;

	/** Favorite brushes for the various states */
	const FSlateBrush* FavoriteHoveredBrush = nullptr;
	const FSlateBrush* FavoriteNotHoveredBrush = nullptr;
	const FSlateBrush* NotFavoriteHoveredBrush = nullptr;
	const FSlateBrush* NotFavoriteNotHoveredBrush = nullptr;
};

void SCapturedPropertiesWidgetTwo::Construct(const FArguments& InArgs)
{
	if (InArgs._PropertyPaths)
	{
		CapturedProperties = *InArgs._PropertyPaths;
	}

	FilteredCapturedProperties = CapturedProperties;

	LoadAndMarkFavorites();

	AddFilters();

    ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(2.f)
		.AutoHeight()
		[
			// Filter bar:
			SAssignNew(FilterBar, SBasicFilterBar<TSharedPtr<FCapturableProperty>>)
			.CustomFilters(Filters)
			.FilterPillStyle(EFilterPillStyle::Basic)
			.bPinAllFrontendFilters(true)
			.bSortFilters(false)
			.OnFilterChanged_Raw(this, &SCapturedPropertiesWidgetTwo::OnFilterChanged)
		]

		+ SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CapturedPropertiesText", "Captured Properties"))
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(PropListView, SListView<TSharedPtr<FCapturableProperty>>)
			.SelectionMode(ESelectionMode::Multi)
			.ListItemsSource(&FilteredCapturedProperties)
			.OnGenerateRow(this, &SCapturedPropertiesWidgetTwo::MakeCapturedPropertyWidget)
			.Visibility(EVisibility::Visible)
			.OnSelectionChanged_Lambda([this](const TSharedPtr<FCapturableProperty>& Item, ESelectInfo::Type Type)
				{
					for (TSharedPtr<FCapturableProperty>& Prop : CapturedProperties)
					{
						Prop->Checked = PropListView->IsItemSelected(Prop);
					}
				})
			.OnItemToString_Debug_Lambda([](const TSharedPtr<FCapturableProperty>& Item)
				{
					return Item->DisplayName;
				})
			.HeaderRow(
				SNew(SHeaderRow)

				+ SHeaderRow::Column(ColumnPath)
				.DefaultLabel(LOCTEXT("PathColumnHeader", "Path"))
				.SortMode(this, &SCapturedPropertiesWidgetTwo::GetSortMode, ColumnPath)
				.SortPriority(this,  &SCapturedPropertiesWidgetTwo::GetSortPriority, ColumnPath)
				.OnSort(this, &SCapturedPropertiesWidgetTwo::HandleSort)
				.FillWidth(0.7f)

				+ SHeaderRow::Column(ColumnProperty)
				.DefaultLabel(LOCTEXT("PropertyColumnHeader", "Property"))
				.SortMode(this, &SCapturedPropertiesWidgetTwo::GetSortMode, ColumnProperty)
				.SortPriority(this,  &SCapturedPropertiesWidgetTwo::GetSortPriority, ColumnProperty)
				.OnSort(this, &SCapturedPropertiesWidgetTwo::HandleSort)
				.FillWidth(0.3f)

				+ SHeaderRow::Column(ColumnFavorite)
				.DefaultLabel(LOCTEXT("FavoriteColumnHeader", "Favorite"))
				.ToolTipText(LOCTEXT("FavoriteColumnHeader", "Favorite"))
				.SortMode(this, &SCapturedPropertiesWidgetTwo::GetSortMode, ColumnFavorite)
				.SortPriority(this, &SCapturedPropertiesWidgetTwo::GetSortPriority, ColumnFavorite)
				.OnSort(this, &SCapturedPropertiesWidgetTwo::HandleSort)
				.HAlignHeader(HAlign_Center)
				.VAlignHeader(VAlign_Center)
				.VAlignCell(VAlign_Center)
				.HAlignCell(HAlign_Center)
				.FixedWidth(24.0f)
				.HeaderContentPadding(FMargin(4.0f, 0, 0, 0))
				[
					SNew(SImage)
					.Image(FVariantManagerStyle::Get().GetBrush("VariantManager.IconFavorite"))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.DesiredSizeOverride(FVector2D(16, 16))
				]
			)
		]
	];

	SortColumn = ColumnFavorite;
	SortMode = EColumnSortMode::Ascending;
	SortPriority = EColumnSortPriority::Primary;

	SortItems();

	PropListView->GetHeaderRow()->RefreshColumns();
}

void SCapturedPropertiesWidgetTwo::HandleSort(
	const EColumnSortPriority::Type NewSortPriority, 
	const FName& ColumnId,
	const EColumnSortMode::Type NewSortMode)
{
	if (NewSortPriority == EColumnSortPriority::Primary)
	{
		SortColumn = ColumnId;
		SortMode = NewSortMode;
		SortPriority = NewSortPriority;
	}

	SortItems();

	if (PropListView)
	{
		PropListView->RequestListRefresh();
	}
}

void SCapturedPropertiesWidgetTwo::SortItems()
{
	auto SortFn = [this](const TSharedPtr<FCapturableProperty>& A, const TSharedPtr<FCapturableProperty>& B)
		{
			if (SortColumn == ColumnFavorite)
			{
				if (A->Favorite != B->Favorite)	// strict-weak-ordering, A=B must return false
				{
					const bool bRet = A->Favorite < B->Favorite;

					return SortMode == EColumnSortMode::Ascending ? bRet : !bRet;
				}
			}
			else if (SortColumn == ColumnPath)
			{
				if (A->SubPath != B->SubPath)
				{
					const bool bRet = A->SubPath < B->SubPath;

					return SortMode == EColumnSortMode::Ascending ? bRet : !bRet;
				}
			}
			else if (SortColumn == ColumnProperty)
			{
				if (A->PropertyName != B->PropertyName)
				{
					const bool bRet = A->PropertyName < B->PropertyName;

					return SortMode == EColumnSortMode::Ascending ? bRet : !bRet;
				}
			}

			return false;
		};
	FilteredCapturedProperties.Sort(SortFn);
}

class FFavoritePropertyItemFilter : public FFilterBase<TSharedPtr<FCapturableProperty>>
{
	FChangedEvent ChangedEvent;
public:
	FFavoritePropertyItemFilter() :
		FFilterBase(MakeShared<FFilterCategory>(LOCTEXT("Title", "Title"), LOCTEXT("Tooltip", "Tooltip")))
	{
	}
	
	virtual void ActiveStateChanged(bool bActive) override { OnChanged().Broadcast(); }
	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FText GetDisplayName() const override { return LOCTEXT("FavoritesFilterText", "Favorites"); }

	virtual FName GetIconName() const override { return NAME_None; }
	virtual FString GetName() const override { return GetDisplayName().ToString(); }
	virtual FText GetToolTipText() const override { return FText::GetEmpty(); }

	virtual bool IsInverseFilter() const override { return false; }
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override {}
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(TSharedPtr<FCapturableProperty> InItem) const override
	{
		return InItem && InItem->Favorite;
	}
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override {}
};

class FActorPropertyItemFilter : public FFilterBase<TSharedPtr<FCapturableProperty>>
{
	FChangedEvent ChangedEvent;
public:
	FActorPropertyItemFilter() :
		FFilterBase(MakeShared<FFilterCategory>(LOCTEXT("Title", "Title"), LOCTEXT("Tooltip", "Tooltip")))
	{
	}
	
	virtual void ActiveStateChanged(bool bActive) override { OnChanged().Broadcast(); }
	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FText GetDisplayName() const override { return LOCTEXT("ActorFilterText", "Actor"); }

	virtual FName GetIconName() const override { return NAME_None; }
	virtual FString GetName() const override { return GetDisplayName().ToString(); }
	virtual FText GetToolTipText() const override { return FText::GetEmpty(); }

	virtual bool IsInverseFilter() const override { return false; }
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override {}
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(TSharedPtr<FCapturableProperty> InItem) const override
	{
		return InItem && InItem->bIsActor;
	}
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override {}
};

class FComponentPropertyItemFilter : public FFilterBase<TSharedPtr<FCapturableProperty>>
{
	FChangedEvent ChangedEvent;
public:
	FComponentPropertyItemFilter() :
		FFilterBase(MakeShared<FFilterCategory>(LOCTEXT("Title", "Title"), LOCTEXT("Tooltip", "Tooltip")))
	{
	}
	
	virtual void ActiveStateChanged(bool bActive) override { OnChanged().Broadcast(); }
	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FText GetDisplayName() const override { return LOCTEXT("ComponentFilterText", "Component"); }

	virtual FName GetIconName() const override { return NAME_None; }
	virtual FString GetName() const override { return GetDisplayName().ToString(); }
	virtual FText GetToolTipText() const override { return FText::GetEmpty(); }

	virtual bool IsInverseFilter() const override { return false; }
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override {}
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(TSharedPtr<FCapturableProperty> InItem) const override
	{
		return InItem && !InItem->bIsActor;
	}
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override {}
};

void SCapturedPropertiesWidgetTwo::AddFilters()
{
	Filters.Add(MakeShared<FFavoritePropertyItemFilter>());
	Filters.Add(MakeShared<FActorPropertyItemFilter>());
	Filters.Add(MakeShared<FComponentPropertyItemFilter>());
}

void SCapturedPropertiesWidgetTwo::LoadAndMarkFavorites()
{
	LoadFavoritesFromIniFile(FavoriteProperties);

	for (TSharedPtr<FCapturableProperty>& CapturedProperty : CapturedProperties)
	{
		TWeakFieldPtr<FProperty> Property = CapturedProperty->Prop.GetLeafMostProperty().Property;

		if (Property.IsValid())
		{
			CapturedProperty->Favorite = FavoriteProperties.Contains(Property->NamePrivate);
		}
	}
}

void SCapturedPropertiesWidgetTwo::SetFavorite(const TSharedPtr<FCapturableProperty>& InProperty, bool bFavorite)
{
	if (InProperty && InProperty->Prop.GetLeafMostProperty().Property.IsValid())
	{
		const FName PropName = InProperty->Prop.GetLeafMostProperty().Property->NamePrivate;

		if (bFavorite)
		{
			FavoriteProperties.Add(PropName);
		}
		else
		{
			FavoriteProperties.Remove(PropName);
		}

		for (TSharedPtr<FCapturableProperty>& CapturedProperty : CapturedProperties)
		{
			TWeakFieldPtr<FProperty> Property = CapturedProperty->Prop.GetLeafMostProperty().Property;
			if (Property.IsValid())
			{
				CapturedProperty->Favorite = FavoriteProperties.Contains(Property->NamePrivate);
			}
		}
	}
}

void SCapturedPropertiesWidgetTwo::SaveFavorites() const
{
	SaveFavoritesToIniFile(FavoriteProperties);
}

class SCapturedPropertiesMultiColumnTableRow : public SMultiColumnTableRow<TSharedPtr<FCapturableProperty>>
{
	SLATE_BEGIN_ARGS(SCapturedPropertiesMultiColumnTableRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FCapturableProperty>, ViewModel)
	SLATE_END_ARGS()

	SCapturedPropertiesWidgetTwo* Parent = nullptr;

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, SCapturedPropertiesWidgetTwo* InParent)
	{
		check(InArgs._ViewModel);

		HighlightText = InArgs._HighlightText;
		ViewModel = InArgs._ViewModel;

		Parent = InParent;

		SMultiColumnTableRow::Construct(
			FSuperRowType::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow")),
			InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == ColumnFavorite)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 4.0f))
				[
					SNew(SFavoriteWidget, ViewModel)
					.IsEnabled_Lambda([this]()
					{
						if (IsSelected())
						{
							return true;
						}

						if (IsHovered() || IsDirectlyHovered())
						{
							return true;
						}

						return false;
					})
					.OnFavoriteChanged_Lambda([this](bool bFavorite)
					{
						if (Parent)
						{
							Parent->SetFavorite(ViewModel, bFavorite);
						}
					})
				];
		}
		else if (InColumnName == ColumnPath)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 4.0f))
				[
					SNew(SHorizontalBox)
					.Visibility(EVisibility::HitTestInvisible)

					+ SHorizontalBox::Slot()
					.Padding(8.0f, 2.0f, 10.0f, 4.0f)
					.MaxWidth(15.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush(ViewModel->bIsActor ? "ClassIcon.Actor" : "ClassIcon.ActorComponent"))
					]

					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::FromString(ViewModel->SubPath))
						.HighlightText(HighlightText)
					]
				];
		}
		else if (InColumnName == ColumnProperty)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 4.0f))
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ViewModel->PropertyName))
					.HighlightText(HighlightText)
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	TAttribute<FText> HighlightText;
	TSharedPtr<FCapturableProperty> ViewModel;
};

TSharedRef<ITableRow> SCapturedPropertiesWidgetTwo::MakeCapturedPropertyWidget(
	TSharedPtr<FCapturableProperty> Item, 
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCapturedPropertiesMultiColumnTableRow, OwnerTable, this)
		.ViewModel(Item)
		.HighlightText(HighlightText);
}

TArray<TSharedPtr<FCapturableProperty>> SCapturedPropertiesWidgetTwo::GetCurrentCheckedProperties()
{
	TArray<TSharedPtr<FCapturableProperty>> Result = CapturedProperties;
	Result.RemoveAll([](const TSharedPtr<FCapturableProperty>& PropCapture)
	{
		return !PropCapture->Checked;
	});

	return Result;
}

void SCapturedPropertiesWidgetTwo::FilterPropertyPaths(const FText& Filter)
{
	HighlightText.Set(Filter);

	OnFilterChanged();
}

void SCapturedPropertiesWidgetTwo::OnFilterChanged()
{
	// Reset selected state of all items:
	for (TSharedPtr<FCapturableProperty>& PropCapture : CapturedProperties)
	{
		PropCapture->Checked = false;
	}

	if (PropListView)
	{
		PropListView->ClearSelection();
	}

	FilteredCapturedProperties.SetNumUninitialized(0, EAllowShrinking::No);

	// Build a list of strings that must be matched
	TArray<FString> FilterStrings;

	FString FilterString = HighlightText.Get().ToString();
	FilterString.TrimStartAndEndInline();
	FilterString.ParseIntoArray(FilterStrings, TEXT(" "), true /*bCullEmpty*/);

	for (TSharedPtr<FCapturableProperty>& PropCapture : CapturedProperties)
	{
		bool bPassedTextFilter = true;

		// check each string in the filter strings list against
		for (const FString& String : FilterStrings)
		{
			if (!PropCapture->DisplayName.Contains(String))
			{
				bPassedTextFilter = false;
				break;
			}
		}

		if (bPassedTextFilter)
		{
			bool bPassesFilterBar = false;
			int ActiveFilters = 0;
			for (const TSharedRef<FFilterBase<TSharedPtr<FCapturableProperty>>>& Filter : Filters)
			{
				if (Filter->IsActive())
				{
					ActiveFilters++;

					if (bool bPassesFilter = Filter->PassesFilter(PropCapture))
					{
						bPassesFilterBar = true;
						break;
					}
				}
			}

			if (ActiveFilters == 0)
			{	// If no filters, just show:
				bPassesFilterBar = true;
			}

			if (bPassesFilterBar)
			{
				FilteredCapturedProperties.Add(PropCapture);
			}
		}
	}

	SortItems();

	if (PropListView)
	{
		PropListView->RebuildList();
	}
}

EColumnSortMode::Type SCapturedPropertiesWidgetTwo::GetSortMode(const FName ColumnName) const
{
	return ColumnName == SortColumn ? SortMode : EColumnSortMode::None;
}

EColumnSortPriority::Type SCapturedPropertiesWidgetTwo::GetSortPriority(const FName ColumnName) const
{
	return ColumnName == SortColumn ? SortPriority : EColumnSortPriority::None;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
