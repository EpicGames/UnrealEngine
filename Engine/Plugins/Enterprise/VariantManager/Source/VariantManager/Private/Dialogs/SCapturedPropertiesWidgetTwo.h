// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCapturedPropertiesWidget.h"
#include "Filters/SBasicFilterBar.h"

#include "Widgets/Views/SHeaderRow.h"

template <typename ItemType> class SListView;

class STableViewBase;
class ITableRow;
struct FCapturableProperty;

class SCapturedPropertiesWidgetTwo : public SCapturedPropertiesWidgetBase
{
public:
	SLATE_BEGIN_ARGS(SCapturedPropertiesWidgetTwo) { }
		SLATE_ARGUMENT(TArray<TSharedPtr<FCapturableProperty>>*, PropertyPaths)
	SLATE_END_ARGS()

	virtual ~SCapturedPropertiesWidgetTwo() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

public:
	virtual TArray<TSharedPtr<FCapturableProperty>> GetCurrentCheckedProperties() override;
	virtual void FilterPropertyPaths(const FText& Filter) override;

	void SetFavorite(const TSharedPtr<FCapturableProperty>& Property, bool bFavorite);
	void SaveFavorites() const;

private:
	EColumnSortMode::Type GetSortMode(const FName ColumnName) const;
	EColumnSortPriority::Type GetSortPriority(FName ColumnName) const;
	void HandleSort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	void SortItems();

	void AddFilters();
	void OnFilterChanged();

	void LoadAndMarkFavorites();

private:
	TSharedRef<ITableRow> MakeCapturedPropertyWidget(TSharedPtr<FCapturableProperty> Item, const TSharedRef<STableViewBase>& OwnerTable);

	TArray<TSharedPtr<FCapturableProperty>> CapturedProperties;

	TSharedPtr<SListView<TSharedPtr<FCapturableProperty>>> PropListView;
	TSharedPtr<SBasicFilterBar<TSharedPtr<FCapturableProperty>>> FilterBar;

	TArray<TSharedPtr<FCapturableProperty>> FilteredCapturedProperties;
	TArray<TSharedRef<FFilterBase<TSharedPtr<FCapturableProperty>>>> Filters;

	TSet<FName> FavoriteProperties;

	TAttribute<FText> HighlightText;

	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;
	EColumnSortPriority::Type SortPriority = EColumnSortPriority::Primary;
	FName SortColumn;
};
