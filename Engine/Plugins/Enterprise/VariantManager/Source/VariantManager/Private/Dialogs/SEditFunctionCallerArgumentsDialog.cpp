// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEditFunctionCallerArgumentsDialog.h"

#include "Editor.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "SlateOptMacros.h"
#include "SPrimaryButton.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SEditFunctionCallerArgumentsDialog"

void SEditFunctionCallerArgumentsDialog::Construct(const FArguments& InArgs, const FName& InFunctionName, const TMap<FName, FString>& InFunctionArguments)
{
	CallerArguments.Reset(NewObject<UEditFunctionCallerArguments>());
	CallerArguments->FunctionName = InFunctionName;
	CallerArguments->Arguments = InFunctionArguments;

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	PropertyView->SetObject(CallerArguments.Get());

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("WindowTitle", "Edit Function Caller Arguments"))
	.AutoCenter(EAutoCenter::PreferredWorkArea)
	.SizingRule(ESizingRule::UserSized)
	.ClientSize(FVector2D(500, 500))
	.IsTopmostWindow(true)
	[
		SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		.Padding(FMargin(10.0f, 10.0f, 10.0f, 10.0f))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				PropertyView.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f)
			[
				SNew(SSpacer)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OkButton", "OK"))
					.OnClicked(this, &SEditFunctionCallerArgumentsDialog::OnDialogConfirmed)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelButton", "Cancel"))
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.OnClicked(this, &SEditFunctionCallerArgumentsDialog::OnDialogCanceled)
				]
			]
		]
	]);
}

TSharedPtr<SEditFunctionCallerArgumentsDialog> 
SEditFunctionCallerArgumentsDialog::OpenDialogAsModalWindow(
	const FName& FunctionName,
	const TMap<FName, FString>& Arguments)
{
	TSharedPtr<SEditFunctionCallerArgumentsDialog> Dialog = SNew(SEditFunctionCallerArgumentsDialog, FunctionName, Arguments);

	GEditor->EditorAddModalWindow(Dialog.ToSharedRef());

	return Dialog;

}

TMap<FName, FString> SEditFunctionCallerArgumentsDialog::GetFunctionArguments() const
{
	if (CallerArguments)
	{
		return CallerArguments->Arguments;
	}

	return {};
}

FReply SEditFunctionCallerArgumentsDialog::OnDialogConfirmed()
{
	bUserAccepted = true;
	RequestDestroyWindow();

	return FReply::Handled();
}

FReply SEditFunctionCallerArgumentsDialog::OnDialogCanceled()
{
	bUserAccepted = false;
	RequestDestroyWindow();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
