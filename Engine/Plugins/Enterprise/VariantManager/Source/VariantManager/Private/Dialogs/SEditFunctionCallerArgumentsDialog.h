// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWindow.h"

#include "SEditFunctionCallerArgumentsDialog.generated.h"

/**
 * Object representation for function caller arguments editing (with IDetailsView)
 */
UCLASS()
class UEditFunctionCallerArguments : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category="Function Caller")
	FName FunctionName;

	UPROPERTY(EditAnywhere, Category="Function Caller")
	TMap<FName, FString> Arguments;
};

/**
 * Dialog for editing function caller arguments
 */
class SEditFunctionCallerArgumentsDialog : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SEditFunctionCallerArgumentsDialog)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FName& InFunctionName, const TMap<FName, FString>& InFunctionArguments);

public:
	static TSharedPtr<SEditFunctionCallerArgumentsDialog> OpenDialogAsModalWindow(const FName& FunctionName, const TMap<FName, FString>& Arguments);

public:
	bool GetUserAccepted() const { return bUserAccepted; }

	TMap<FName, FString> GetFunctionArguments() const;

private:
	FReply OnDialogConfirmed();
	FReply OnDialogCanceled();

private:
	TSharedPtr<class IDetailsView> PropertyView;
	TStrongObjectPtr<UEditFunctionCallerArguments> CallerArguments;

	bool bUserAccepted = false;
};
