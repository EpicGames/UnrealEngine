// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerActorNode.h"

#include "ISceneOutliner.h"
#include "SVariantManager.h"
#include "Variant.h"
#include "VariantManager.h"
#include "VariantManagerDragDropOp.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerSelection.h"
#include "VariantObjectBinding.h"

#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerModule.h"
#include "ActorTreeItem.h"
#include "ScopedTransaction.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "FVariantManagerActorNode"

FVariantManagerActorNode::FVariantManagerActorNode(UVariantObjectBinding* InObjectBinding, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerDisplayNode(InParentNode, nullptr)
	, ObjectBinding(InObjectBinding)
	, VariantManager(InVariantManager)
{
}

TWeakObjectPtr<UVariantObjectBinding> FVariantManagerActorNode::GetObjectBinding() const
{
	return ObjectBinding;
}

void FVariantManagerActorNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FVariantManagerDisplayNode::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection("Edit", LOCTEXT("ActorEditSectionText", "Edit"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RemoveActorBindings);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Actor", LOCTEXT("ActorSectionText", "Actor"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().AddPropertyCaptures);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().AddFunction);

	int32 NumSelectedNodes = 0;
	if (TSharedPtr<FVariantManager> PinnedVariantManager = VariantManager.Pin())
	{
		FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
		NumSelectedNodes = Selection.GetSelectedActorNodes().Num();
	}

	if (NumSelectedNodes > 1)
	{
		MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RebindActorDisabled);
	}
	else
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("RebindActorName", "Rebind to other Actor"),
			LOCTEXT("RebindActorName_Tooltip", "Connect this binding to a different actor and try reusing the captured properties"),
			FNewMenuDelegate::CreateSP(this, &FVariantManagerActorNode::AddAssignActorSubMenu));
	}

	MenuBuilder.EndSection();
}

FText FVariantManagerActorNode::GetDisplayNameToolTipText() const
{
	if (const UClass* BindingClass = GetClassForObjectBinding())
	{
		return FText::FromString(BindingClass->GetName());
	}

	return LOCTEXT("FailedToResolveTooltip", "Binding can't be resolved. Right-click to rebind to another actor");
}

const FSlateBrush* FVariantManagerActorNode::GetIconBrush() const
{
	if (const UClass* BindingClass = GetClassForObjectBinding())
	{
		return FSlateIconFinder::FindIconBrushForClass(BindingClass);
	}

	return FAppStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
}

const FSlateBrush* FVariantManagerActorNode::GetIconOverlayBrush() const
{
	return nullptr;
}

EVariantManagerNodeType FVariantManagerActorNode::GetType() const
{
	return EVariantManagerNodeType::Actor;
}

FText FVariantManagerActorNode::GetDisplayName() const
{
	FText NewDisplayText = ObjectBinding->GetDisplayText();

	// Update the property list if our display name changed
	// This will misfire when renaming, which is rare. It will make sure
	// we stop showing properties if our actor doesn't resolve anymore, though
	if (NewDisplayText.ToString() != OldDisplayText.ToString())
	{
		if (TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin())
		{
			VarMan->GetVariantManagerWidget()->RefreshPropertyList();
		}
		OldDisplayText = NewDisplayText;
	}

	return NewDisplayText;
}

FSlateColor FVariantManagerActorNode::GetDisplayNameColor() const
{
	if (GetClassForObjectBinding())
	{
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	}

	return FSlateColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f));
}

void FVariantManagerActorNode::SetDisplayName(const FText& NewDisplayName)
{
	UObject* Obj = ObjectBinding->GetObject();

	if (AActor* ObjAsActor = Cast<AActor>(Obj))
	{
		ObjAsActor->SetActorLabel(*NewDisplayName.ToString());
	}
	else
	{
		Obj->Rename(*NewDisplayName.ToString());
	}
}

bool FVariantManagerActorNode::IsSelectable() const
{
	return true;
}

bool FVariantManagerActorNode::CanDrag() const
{
	return true;
}

TWeakPtr<FVariantManager> FVariantManagerActorNode::GetVariantManager() const
{
	return VariantManager;
}

TOptional<EItemDropZone> FVariantManagerActorNode::CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	if (ActorDragDrop.IsValid())
	{
		UVariant* Var = GetObjectBinding()->GetParent();
		TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;

		VarMan->CanAddActorsToVariant(ActorDragDrop->Actors, Var, ActorsWeCanAdd);
		int32 NumActorsWeCanAdd = ActorsWeCanAdd.Num();

		// Get the FDecoratedDragDropOp so that we can use its non-virtual SetToolTip function and not FActorDragDropGraphEdOp's
		// We know this is valid because FActorDragDropGraphEdOp derives from it
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();

		if (NumActorsWeCanAdd > 0)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_BindActors", "Bind {0} {0}|plural(one=actor,other=actors) to variant '{1}'"),
				NumActorsWeCanAdd,
				Var->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
		}
		else
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ActorsAlreadyBound", "Actors already bound to variant '{0}'!"),
				Var->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return TOptional<EItemDropZone>();
		}
	}
	else if (VarManDragDrop.IsValid())
	{
		// Get all dragged bindings, but only keeping one for each actor
		TArray<UVariantObjectBinding*> DraggedBindings;
		TArray<UObject*> DraggedActors;
		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				if (TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode))
				{
					if (UVariantObjectBinding* DraggedBinding = DraggedActorNode->GetObjectBinding().Get())
					{
						UObject* DraggedActor = DraggedBinding->GetObject();
						if (!DraggedActors.Contains(DraggedActor))
						{
							DraggedBindings.Add(DraggedBinding);
							DraggedActors.Add(DraggedActor);
						}
					}
				}
			}
		}

		int32 NumBindingsWeCanCopy = 0;
		int32 NumOwnBindings = 0;
		UVariant* Var = GetObjectBinding()->GetParent();

		// We know we'll be able to reorder bindings within a variant, so discard those
		for (int32 Index = DraggedBindings.Num() - 1; Index >= 0; Index--)
		{
			if (DraggedBindings[Index]->GetParent() == Var)
			{
				DraggedBindings.RemoveAt(Index);
				DraggedActors.RemoveAt(Index);
				NumOwnBindings += 1;
			}
		}

		// For the remaining foreign bindings, see if our variant already has binding to those actors
		TArray<TWeakObjectPtr<AActor>> ActorsToCheck;
		for (UObject* Actor : DraggedActors)
		{
			if (AActor* ActualActor = Cast<AActor>(Actor))
			{
				ActorsToCheck.Add(ActualActor);
			}
		}
		TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;
		VarMan->CanAddActorsToVariant(ActorsToCheck, Var, ActorsWeCanAdd);
		NumBindingsWeCanCopy += ActorsWeCanAdd.Num();

		// Get the FDecoratedDragDropOp so that we can use its non-virtual SetToolTip function and not FActorDragDropGraphEdOp's
		// We know this is valid because FActorDragDropGraphEdOp derives from it
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();

		FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
		bool bIsCopy = ModifierKeysState.IsControlDown();

		// Can copy new bindings
		if (NumBindingsWeCanCopy > 0 && bIsCopy)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_CopyActorBindings", "Copy {0} actor {0}|plural(one=binding,other=bindings) to variant '{1}'"),
				NumBindingsWeCanCopy,
				Var->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
		}
		// Have at least one binding we can move
		else if ((NumOwnBindings+NumBindingsWeCanCopy) > 0 && !bIsCopy)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_MoveActorBindings", "Move {0} actor {0}|plural(one=binding,other=bindings) to variant '{1}'"),
				NumOwnBindings + NumBindingsWeCanCopy,
				Var->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
		}
		// If we at least were dragging some bindings (or else we were dragging variants/variant sets, etc)
		else if(DraggedBindings.Num() > 0)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_AllBindingsAlreadyExist", "All bindings already exist on variant '{0}'!"),
				Var->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return TOptional<EItemDropZone>();
		}
	}

	if (VarManDragDrop.IsValid())
	{
		VarManDragDrop->ResetToDefaultToolTip();
	}
	else if (ActorDragDrop.IsValid())
	{
		ActorDragDrop->ResetToDefaultToolTip();
	}
	return TOptional<EItemDropZone>();
}

void FVariantManagerActorNode::Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone)
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return;
	}

	if (ActorDragDrop.IsValid())
	{
		UVariant* Var = GetObjectBinding()->GetParent();

		TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;
		VarMan->CanAddActorsToVariant(ActorDragDrop->Actors, Var, ActorsWeCanAdd);

		int32 NumActorsWeCanAdd = ActorsWeCanAdd.Num();
		if (NumActorsWeCanAdd > 0)
		{
			TArray<AActor*> Actors;
			Actors.Reserve(ActorDragDrop->Actors.Num());
			for (TWeakObjectPtr<AActor> Actor : ActorDragDrop->Actors)
			{
				AActor* RawActor = Actor.Get();
				if (RawActor)
				{
					Actors.Add(RawActor);
				}
			}

			int32 TargetIndex = Var->GetBindingIndex(GetObjectBinding().Get());
			if (TargetIndex != INDEX_NONE && ItemDropZone != EItemDropZone::AboveItem)
			{
				TargetIndex += 1;
			}

			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("ActorNodeDropSceneActors", "Drop {0} scene {0}|plural(one=actor,other=actors) near actor binding '{1}'"),
				Actors.Num(),
				GetDisplayName()));

			VarMan->CreateObjectBindingsAndCaptures(Actors, {Var}, TargetIndex);

			VarMan->GetVariantManagerWidget()->RefreshActorList();
		}
	}
	else if (VarManDragDrop.IsValid())
	{
		// Get all dragged bindings, but only keeping one for each actor
		TArray<UVariantObjectBinding*> DraggedBindings;
		TArray<UObject*> DraggedActors;
		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode);
				if (DraggedActorNode.IsValid())
				{
					UVariantObjectBinding* DraggedBinding = DraggedActorNode->GetObjectBinding().Get();
					if (DraggedBinding)
					{
						UObject* DraggedActor = DraggedBinding->GetObject();
						if (!DraggedActors.Contains(DraggedActor))
						{
							DraggedBindings.Add(DraggedBinding);
							DraggedActors.Add(DraggedActor);
						}
					}
				}
			}
		}

		UVariant* Var = GetObjectBinding()->GetParent();

		int32 TargetIndex = Var->GetBindingIndex(GetObjectBinding().Get());
		if (TargetIndex != INDEX_NONE && ItemDropZone != EItemDropZone::AboveItem)
		{
			TargetIndex += 1;
		}

		FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

		TSet<UObject*> BoundObjects;
		for (UVariantObjectBinding* Binding : Var->GetBindings())
		{
			BoundObjects.Add(Binding->GetObject());
		}

		// Copy UVariantObjectBindings
		if(ModifierKeysState.IsControlDown())
		{
			// Remove bindings to actors our variant already has
			TArray<UVariantObjectBinding*> BindingsWeCanDuplicate;
			for (UVariantObjectBinding* DraggedBinding : DraggedBindings)
			{
				UObject* DraggedObject = DraggedBinding->GetObject();
				if (DraggedObject && !BoundObjects.Contains(DraggedObject))
				{
					BindingsWeCanDuplicate.Add(DraggedBinding);
				}
			}

			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("ActorNodeDropActors", "Drop {0} actor {0}|plural(one=binding,other=bindings) near actor binding '{1}'"),
				BindingsWeCanDuplicate.Num(),
				GetDisplayName()));

			VarMan->DuplicateObjectBindings(BindingsWeCanDuplicate, Var, TargetIndex);
		}
		// Move UVariantObjectBindings
		else
		{
			// Remove foreign bindings to actors our variant already has
			// We might be just moving one of our own bindings
			TArray<UVariantObjectBinding*> BindingsWeCanMove;
			for (UVariantObjectBinding* DraggedBinding : DraggedBindings)
			{
				UObject* DraggedObject = DraggedBinding->GetObject();
				if (DraggedObject && (!BoundObjects.Contains(DraggedObject) || DraggedBinding->GetParent() == Var))
				{
					BindingsWeCanMove.Add(DraggedBinding);
				}
			}

			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("ActorNodeDropActors", "Drop {0} actor {0}|plural(one=binding,other=bindings) near actor binding '{1}'"),
				BindingsWeCanMove.Num(),
				GetDisplayName()));

			VarMan->AddObjectBindings(BindingsWeCanMove, Var, TargetIndex, true);
		}

		VarMan->GetVariantManagerWidget()->RefreshActorList();
	}
}

const UClass* FVariantManagerActorNode::GetClassForObjectBinding() const
{
	if (ObjectBinding.IsValid())
	{
		if (UObject* Obj = ObjectBinding->GetObject())
		{
			return Obj->GetClass();
		}
	}

	return nullptr;
}

void FVariantManagerActorNode::AddAssignActorSubMenu(FMenuBuilder& MenuBuilder)
{
	// Copied from FSequencer::AssignActor

	// If we're showing this menu, we know for a fact only our actor node is selected,
	// so we only have to check our variant
	TSet<const AActor*> BoundActors;
	if (ObjectBinding.IsValid())
	{
		if (UVariant* ParentVariant = ObjectBinding->GetParent())
		{
			for (UVariantObjectBinding* Binding : ParentVariant->GetBindings())
			{
				if (AActor* BoundActor = Cast<AActor>(Binding->GetObject()))
				{
					BoundActors.Add(BoundActor);
				}
			}
		}
	}

	auto IsActorValidForAssignment = [BoundActors](const AActor* InActor)
	{
		return !BoundActors.Contains(InActor);
	};

	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RebindToSelected,
							 NAME_None,
							 TAttribute<FText>(),
							 TAttribute<FText>(this, &FVariantManagerActorNode::GetRebindToSelectedTooltip));

	// Set up a menu entry to assign an actor to the object binding node
	FSceneOutlinerInitializationOptions InitOptions;
	{
		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		// Only want the actor label column
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		// Only display actors that are not possessed already
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda( IsActorValidForAssignment ));
	}

	// actor selector to allow the user to choose an actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef< SWidget > MiniSceneOutliner =
		SNew( SBox )
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([&](AActor* Actor)
				{
					FSlateApplication::Get().DismissAllMenus();

					if (Actor && ObjectBinding.IsValid())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("RebindToActorTransaction", "Rebind variant to actor '{0}'"),
							FText::FromString(Actor->GetActorLabel())));

						ObjectBinding->SetObject(Actor);
					}
				})
			)
		];

	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);
}

FText FVariantManagerActorNode::GetRebindToSelectedTooltip() const
{
	if (TSharedPtr<FVariantManager> PinnedVariantManager = VariantManager.Pin())
	{
		if(TSharedPtr<SVariantManager> VariantManagerWidget = PinnedVariantManager->GetVariantManagerWidget())
		{
			UVariantObjectBinding* SelectedBinding = nullptr;
			UObject* SelectedObject = nullptr;
			VariantManagerWidget->GetSelectedBindingAndEditorActor(SelectedBinding, SelectedObject);

			AActor* SelectedActor = Cast<AActor>(SelectedObject);

			if (SelectedBinding && SelectedActor)
			{
				return FText::Format(LOCTEXT("RebindToThisActorTooltipFound", "Rebind to '{0}'"), FText::FromString(SelectedActor->GetActorLabel()));
			}
		}
	}

	// If we reach here, then so did SVariantManager::CanRebindToSelectedActor, so the button will be disabled
	return LOCTEXT("RebindToThisActorTooltipDisabled", "Cannot rebind to this selected actor");
}

TSharedRef<SWidget> FVariantManagerActorNode::GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow)
{
	return
	SNew(SBox)
	[
		SNew(SBorder)
		.VAlign(VAlign_Center)
		.BorderImage(this, &FVariantManagerDisplayNode::GetNodeBorderImage)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin( 4.0, 0.0f, 0.0f, 0.0f ))
			.HeightOverride(26)
			.ToolTipText(this, &FVariantManagerActorNode::GetDisplayNameToolTipText)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
				[
					SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage)
						.Image(this, &FVariantManagerActorNode::GetIconBrush)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(EditableLabel, SInlineEditableTextBlock)
					.IsReadOnly(this, &FVariantManagerDisplayNode::IsReadOnly)
					.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
					.OnTextCommitted(this, &FVariantManagerDisplayNode::HandleNodeLabelTextChanged)
					.Text(this, &FVariantManagerDisplayNode::GetDisplayName)
					.ToolTipText(this, &FVariantManagerDisplayNode::GetDisplayNameToolTipText)
					.Clipping(EWidgetClipping::ClipToBounds)
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
