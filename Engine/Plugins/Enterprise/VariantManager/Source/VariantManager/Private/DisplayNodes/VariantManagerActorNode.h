// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerDisplayNode.h"


class FDragDropEvent;
class FMenuBuilder;
class SVariantManagerTableRow;
class UVariantObjectBinding;
enum class EItemDropZone;
struct FSlateBrush;

/**
* A node for displaying an object binding
*/
class FVariantManagerActorNode : public FVariantManagerDisplayNode
{
public:
	FVariantManagerActorNode(UVariantObjectBinding* InObjectBinding, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManager> InVariantManager);

	/** Gets the UVariantObjectBinding held by this actor node */
	TWeakObjectPtr<UVariantObjectBinding> GetObjectBinding() const;

	// Begin FVariantManagerDisplayNode interface
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual FText GetDisplayNameToolTipText() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual const FSlateBrush* GetIconOverlayBrush() const override;
	virtual EVariantManagerNodeType GetType() const override;
	virtual FText GetDisplayName() const override;
	virtual FSlateColor GetDisplayNameColor() const override;
	virtual void SetDisplayName(const FText& NewDisplayName) override;
	virtual bool IsSelectable() const override;
	virtual bool CanDrag() const override;
	virtual TWeakPtr<FVariantManager> GetVariantManager() const override;
	virtual TOptional<EItemDropZone> CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const override;
	virtual void Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) override;
	virtual TSharedRef<SWidget> GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow) override;
	// End FVariantManagerDisplayNode interface

private:
	/** Gets the tooltip to show on the 'RebindToSelected' button */
	FText GetRebindToSelectedTooltip() const;

	/** Gets the class of the UObject that is bound to our ObjectBinding, or nullptr */
	const UClass* GetClassForObjectBinding() const;

	/** Creates the floating submenu that is shown when mousing over the "Rebind to other actor" right-click option */
	void AddAssignActorSubMenu(FMenuBuilder& MenuBuilder);

	TWeakObjectPtr<UVariantObjectBinding> ObjectBinding;
	mutable FText OldDisplayText;
	FText DefaultDisplayName;
	TWeakPtr<FVariantManager> VariantManager;
};
