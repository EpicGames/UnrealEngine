// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerColorPropertyNode.h"

#include "Engine/Engine.h"
#include "PropertyValue.h"
#include "VariantManagerLog.h"

#include "ScopedTransaction.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FVariantManagerColorPropertyNode"

namespace FVariantColorNodeImpl
{
	enum class EGetCommonColorResult : uint8
	{
		OK,
		Failed,
		MultipleValues,
	};

	EGetCommonColorResult GetCommonColorFromPropertyValues(FLinearColor& OutColor, const TArray<TWeakObjectPtr<UPropertyValue>>& PropertyValues)
	{
		// Find a PropertyValue that has valid recorded data
		UPropertyValue* ValidPropVal = nullptr;
		for (const TWeakObjectPtr<UPropertyValue>& PropVal : PropertyValues)
		{
			UPropertyValue* PropertyValue = PropVal.Get();
			if (!PropertyValue)
			{
				continue;
			}

			if (!PropertyValue->HasRecordedData())
			{
				PropertyValue->RecordDataFromResolvedObject();
			}

			// Check again after we tried to record
			if (PropertyValue->HasRecordedData())
			{
				ValidPropVal = PropertyValue;
				break;
			}
		}
		if (!ValidPropVal)
		{
			return EGetCommonColorResult::Failed;
		}

		// Check if we have multiple values
		const TArray<uint8>& FirstRecordedData = ValidPropVal->GetRecordedData();
		for (const TWeakObjectPtr<UPropertyValue>& PropVal : PropertyValues)
		{
			UPropertyValue* PropertyValue = PropVal.Get();
			if (PropertyValue && (PropertyValue->GetRecordedData() != FirstRecordedData))
			{
				return EGetCommonColorResult::MultipleValues;
			}
		}

		UScriptStruct* Struct = ValidPropVal->GetStructPropertyStruct();
		if (!Struct)
		{
			return EGetCommonColorResult::Failed;
		}

		FName StructName = Struct->GetFName();
		if (StructName == NAME_Color && FirstRecordedData.Num() == sizeof(FColor))
		{
			// Convert to linear here as this will be shown on the color picker
			FColor Result = *((FColor*)FirstRecordedData.GetData());
			OutColor = FLinearColor(Result);

			return EGetCommonColorResult::OK;
		}
		else if (StructName == NAME_LinearColor && FirstRecordedData.Num() == sizeof(FLinearColor))
		{
			OutColor = *((FLinearColor*)FirstRecordedData.GetData());

			return EGetCommonColorResult::OK;
		}

		return EGetCommonColorResult::Failed;
	}
}

FVariantManagerColorPropertyNode::FVariantManagerColorPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(InPropertyValues, InVariantManager)
{
}

FReply FVariantManagerColorPropertyNode::OnClickColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bOnlyRefreshOnOk = true;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FVariantManagerColorPropertyNode::OnSetColorFromColorPicker);
	}

	FVariantColorNodeImpl::GetCommonColorFromPropertyValues(PickerArgs.InitialColor, PropertyValues);

	OpenColorPicker(PickerArgs);

	return FReply::Handled();
}

void FVariantManagerColorPropertyNode::OnSetColorFromColorPicker(FLinearColor NewColor)
{
	CachedColor = NewColor;

	UPropertyValue* ValidPropVal = nullptr;
	for (const TWeakObjectPtr<UPropertyValue>& PropVal : PropertyValues)
	{
		if (UPropertyValue* PropertyValue = PropVal.Get())
		{
			ValidPropVal = PropertyValue;
			break;
		}
	}
	if (!ValidPropVal)
	{
		return;
	}

	UScriptStruct* Struct = ValidPropVal->GetStructPropertyStruct();
	FName StructName = Struct->GetFName();

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("SetColorProperty", "Edit captured property '{0}'"),
					  FText::FromName(ValidPropVal->GetPropertyName())));

	if (StructName == NAME_Color)
	{
		FColor Color = NewColor.ToFColor(false);

		for (TWeakObjectPtr<UPropertyValue> PropVal : PropertyValues)
		{
			if (UPropertyValue* PropertyValue = PropVal.Get())
			{
				PropertyValue->SetRecordedData((uint8*)&Color, sizeof(FColor));
			}
		}
	}
	else if (StructName == NAME_LinearColor)
	{
		for (TWeakObjectPtr<UPropertyValue> PropVal : PropertyValues)
		{
			if (UPropertyValue* PropertyValue = PropVal.Get())
			{
				PropertyValue->SetRecordedData((uint8*)&NewColor, sizeof(FLinearColor));
			}
		}
	}
}

TSharedPtr<SWidget> FVariantManagerColorPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no UPropertyValues!");
		return SNullWidget::NullWidget;
	}

	// Try checking if all our property values have the same color
	FVariantColorNodeImpl::EGetCommonColorResult ColorResult =
		FVariantColorNodeImpl::GetCommonColorFromPropertyValues(CachedColor, PropertyValues);

	if (ColorResult == FVariantColorNodeImpl::EGetCommonColorResult::Failed)
	{
		UPropertyValue* ValidPropVal = nullptr;
		for (const TWeakObjectPtr<UPropertyValue>& PropVal : PropertyValues)
		{
			if (UPropertyValue* PropertyValue = PropVal.Get())
			{
				ValidPropVal = PropertyValue;
				break;
			}
		}
		return GetFailedToResolveWidget(ValidPropVal);
	}
	else if (ColorResult == FVariantColorNodeImpl::EGetCommonColorResult::MultipleValues)
	{
		return GetMultipleValuesWidget();
	}

	// Taken from FColorStructCustomization::CreateColorWidget.  Todo this should be combined into one color block based on changes to FColorStructCustomization
	return	SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f)) // Extra padding to match the internal padding of single property nodes
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					// Displays the color with alpha unless it is ignored
					SNew(SColorBlock)
					.Color_Lambda([&](){ return CachedColor; })
					.ShowBackgroundForAlpha(true)
					.OnMouseButtonDown(this, &FVariantManagerColorPropertyNode::OnClickColorBlock)
					.Size(FVector2D(35.0f, 17.0f))
					.IsEnabled(true)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					// Displays the color without alpha
					SNew(SColorBlock)
					.Color_Lambda([&](){ return CachedColor; })
					.ShowBackgroundForAlpha(false)
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.OnMouseButtonDown(this, &FVariantManagerColorPropertyNode::OnClickColorBlock)
					.Size(FVector2D(35.0f, 17.0f))
				]
			];
}

#undef LOCTEXT_NAMESPACE
