// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"


class UPropertyValue;

class FVariantManagerColorPropertyNode : public FVariantManagerPropertyNode
{
public:
	FVariantManagerColorPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);

protected:
	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

private:
	FReply OnClickColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void OnSetColorFromColorPicker(FLinearColor NewColor);

	// Keep a local cache as applying/recording from UPropertyValues is not free
	// and the SColorBlock reads from this every frame
	FLinearColor CachedColor = FLinearColor::Black;
};
