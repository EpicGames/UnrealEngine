// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerDisplayNode.h"
#include "VariantManager.h"
#include "VariantManagerNodeTree.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantManagerDisplayNode)

#define LOCTEXT_NAMESPACE "VariantManagerDisplayNode"


FVariantManagerDisplayNode::FVariantManagerDisplayNode(TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree )
	: VirtualTop( 0.f )
	, VirtualBottom( 0.f )
	, ParentNode( InParentNode )
	, ParentTree( InParentTree )
	, bExpanded( false )
	, bSelected( false )
{
	BackgroundBrush = nullptr;
}

const FTableRowStyle* FVariantManagerDisplayNode::GetRowStyle() const
{
	return &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row");
}

FSlateColor FVariantManagerDisplayNode::GetDisplayNameColor() const
{
	return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
}

FText FVariantManagerDisplayNode::GetDisplayNameToolTipText() const
{
	return FText();
}

void FVariantManagerDisplayNode::HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType)
{
	SetDisplayName(NewLabel);
}

FReply FVariantManagerDisplayNode::OnDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Unhandled();
}

TSharedRef<SWidget> FVariantManagerDisplayNode::GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow)
{
	return
	SNew(SBox)
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	.HeightOverride(13)
	[
		SNew(SBorder)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
	];
}

const FSlateBrush* FVariantManagerDisplayNode::GetIconBrush() const
{
	return nullptr;
}

const FSlateBrush* FVariantManagerDisplayNode::GetIconOverlayBrush() const
{
	return nullptr;
}

FSlateColor FVariantManagerDisplayNode::GetIconColor() const
{
	return FSlateColor( FLinearColor::White );
}

FText FVariantManagerDisplayNode::GetIconToolTipText() const
{
	return FText();
}

const FSlateBrush* FVariantManagerDisplayNode::GetNodeBorderImage() const
{
	return BackgroundBrush;
}

void FVariantManagerDisplayNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{

}

TWeakPtr<FVariantManager> FVariantManagerDisplayNode::GetVariantManager() const
{
	TWeakPtr<FVariantManagerNodeTree> WeakTree = GetParentTree();
	if (WeakTree.IsValid())
	{
		return TWeakPtr<FVariantManager>(WeakTree.Pin()->GetVariantManager().AsShared());
	}

	return nullptr;
}

void FVariantManagerDisplayNode::SetExpansionState(bool bInExpanded)
{

}

bool FVariantManagerDisplayNode::IsExpanded() const
{
	return bExpanded;
}

bool FVariantManagerDisplayNode::IsHidden() const
{
	if (ParentTree.IsValid())
	{
		return ParentTree.Pin()->HasActiveFilter() && !ParentTree.Pin()->IsNodeFiltered(AsShared());
	}
	return true;
}

bool FVariantManagerDisplayNode::IsHovered() const
{
	if (ParentTree.IsValid())
	{
		return ParentTree.Pin()->GetHoveredNode().Get() == this;
	}
	return false;
}

void FVariantManagerDisplayNode::HandleContextMenuRenameNodeExecute()
{
	RenameRequestedEvent.Broadcast();
}

bool FVariantManagerDisplayNode::HandleContextMenuRenameNodeCanExecute() const
{
	return !IsReadOnly();
}


#undef LOCTEXT_NAMESPACE
