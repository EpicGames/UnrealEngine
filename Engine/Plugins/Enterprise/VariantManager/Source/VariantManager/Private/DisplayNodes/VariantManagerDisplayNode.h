// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Text/SInlineEditableTextBlock.h"

//#include "DisplayNodes/SVariantManagerEditableLabel.h"
struct FTableRowStyle;

#include "VariantManagerDisplayNode.generated.h"

enum class EItemDropZone;
class FMenuBuilder;
class FVariantManager;
class FVariantManagerNodeTree;
class SVariantManagerTableRow;
enum class EItemDropZone;
class FDragDropEvent;


UENUM()
enum class EVariantManagerNodeType : uint8
{
	VariantSet,
	Variant,
	Actor,
	Property,
	Function,
	Spacer
};

/**
 * Base VariantManager layout node.
 */
class FVariantManagerDisplayNode
	: public TSharedFromThis<FVariantManagerDisplayNode>
{
public:

	FVariantManagerDisplayNode(TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree );

	virtual ~FVariantManagerDisplayNode()= default;

	virtual EVariantManagerNodeType GetType() const
	{
		return EVariantManagerNodeType::Spacer;
	}

	virtual bool IsSelectable() const
	{
		return false;
	}

	virtual bool IsReadOnly() const
	{
		return true;
	}
	virtual void StartRenaming() const
	{
		EditableLabel->EnterEditingMode();
	}
	virtual FText GetDisplayName() const
	{
		return FText();
	}

	virtual const FTableRowStyle* GetRowStyle() const;
	virtual FSlateColor GetDisplayNameColor() const;
	virtual FText GetDisplayNameToolTipText() const;
	virtual void SetDisplayName(const FText& NewDisplayName) {}
	virtual void HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType);

	virtual bool IsResizable() const
	{
		return false;
	}
	virtual void Resize(float NewSize)
	{

	}

	virtual FReply OnDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	virtual TSharedRef<SWidget> GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow);

	virtual const FSlateBrush* GetIconBrush() const;
	virtual const FSlateBrush* GetIconOverlayBrush() const;
	virtual FSlateColor GetIconColor() const;
	virtual FText GetIconToolTipText() const;
	virtual const FSlateBrush* GetNodeBorderImage() const;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder);

	uint32 GetNumChildren() const
	{
		return ChildNodes.Num();
	}
	const TArray<TSharedRef<FVariantManagerDisplayNode>>& GetChildNodes() const
	{
		return ChildNodes;
	}
	void AddChild(TSharedRef<FVariantManagerDisplayNode> NewChild)
	{
		ChildNodes.Add(NewChild);
	}

	template <class PREDICATE_CLASS>
	void SortChildNodes(const PREDICATE_CLASS& Predicate)
	{
		ChildNodes.StableSort(Predicate);

		for (const TSharedRef<FVariantManagerDisplayNode>& ChildNode : ChildNodes)
		{
			ChildNode->SortChildNodes(Predicate);
		}
	}
	TSharedPtr<FVariantManagerDisplayNode> GetParent() const
	{
		return ParentNode.Pin();
	}
	void SetParent(TSharedPtr<FVariantManagerDisplayNode> NewParent)
	{
		ParentNode = NewParent;
	}
	TSharedRef<FVariantManagerDisplayNode> GetOutermostParent()
	{
		TSharedPtr<FVariantManagerDisplayNode> Parent = ParentNode.Pin();
		return Parent.IsValid() ? Parent->GetOutermostParent() : AsShared();
	}

	virtual TWeakPtr<FVariantManager> GetVariantManager() const;

	TWeakPtr<FVariantManagerNodeTree> GetParentTree() const
	{
		return ParentTree;
	}

	virtual void SetExpansionState(bool bInExpanded);
	bool IsExpanded() const;
	bool IsHidden() const;
	bool IsHovered() const;
	bool IsSelected() const
	{
		return bSelected;
	}

	void SetSelected(bool bShouldBeSelected)
	{
		bSelected = bShouldBeSelected;
	}

	float GetVirtualTop() const
	{
		return VirtualTop;
	}
	float GetVirtualBottom() const
	{
		return VirtualBottom;
	}

	DECLARE_EVENT(FVariantManagerDisplayNode, FRequestRenameEvent);
	FRequestRenameEvent& OnRenameRequested() { return RenameRequestedEvent; }

	virtual bool CanDrag() const
	{
		return false;
	}
	virtual TOptional<EItemDropZone> CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const
	{
		return TOptional<EItemDropZone>();
	}
	virtual void Drop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
	{
	}

private:

	/** Callback for executing a "Rename Node" context menu action. */
	void HandleContextMenuRenameNodeExecute();

	/** Callback for determining whether a "Rename Node" context menu action can execute. */
	bool HandleContextMenuRenameNodeCanExecute() const;

protected:
	/** The virtual offset of this item from the top of the tree, irrespective of expansion states. */
	float VirtualTop;

	/** The virtual offset + virtual height of this item, irrespective of expansion states. */
	float VirtualBottom;

	/** The parent of this node*/
	TWeakPtr<FVariantManagerDisplayNode> ParentNode;

	/** List of children belonging to this node */
	TArray<TSharedRef<FVariantManagerDisplayNode>> ChildNodes;

	/** Parent tree that this node is in */
	TWeakPtr<FVariantManagerNodeTree> ParentTree;

	/** Holds the editable text label widget. */
	TSharedPtr<SInlineEditableTextBlock> EditableLabel;

	/** Whether or not the node is expanded */
	bool bExpanded;

	bool bSelected;

	/** Event that is triggered when rename is requested */
	FRequestRenameEvent RenameRequestedEvent;

	/** Default background brush for this node when expanded */
	const FSlateBrush* BackgroundBrush;
};
