// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerEnumPropertyNode.h"

#include "Framework/Views/TableViewMetadata.h"
#include "PropertyValue.h"
#include "SVariantManager.h"
#include "VariantManager.h"
#include "VariantManagerLog.h"

#include "IDocumentation.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FVariantManagerEnumPropertyNode"

FVariantManagerEnumPropertyNode::FVariantManagerEnumPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(InPropertyValues, InVariantManager)
{
}

TSharedPtr<SWidget> FVariantManagerEnumPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no UPropertyValues!");
		return SNew(SBox);
	}

	// Check to see if we have all valid, equal UPropertyValues
	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();
	uint32 FirstPropHash = FirstPropertyValue->GetPropertyPathHash();
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (!PropertyValue.IsValid())
		{
			UE_LOGF(LogVariantManager, Error, "PropertyValue was invalid!");
			return SNew(SBox);
		}

		if (PropertyValue.Get()->GetPropertyPathHash() != FirstPropHash)
		{
			UE_LOGF(LogVariantManager, Error, "A PropertyNode's PropertyValue array describes properties with different paths!");
			return SNew(SBox);
		}
	}

	// If all properties fail to resolve, just give back a "Failed to resolve" text block
	bool bAtLeastOneResolved = false;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		UPropertyValue* PropValRaw = WeakPropertyValue.Get();
		if (PropValRaw->Resolve())
		{
			if (!PropValRaw->HasRecordedData())
			{
				PropValRaw->RecordDataFromResolvedObject();
			}

			bAtLeastOneResolved = true;
		}
	}
	if(!bAtLeastOneResolved)
	{
		return GetFailedToResolveWidget(FirstPropertyValue);
	}

	// If properties have different values, just give back a "Multiple Values" text block
	bool bSameValue = true;
	const TArray<uint8>& FirstRecordedData = FirstPropertyValue->GetRecordedData();
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		const TArray<uint8>& RecordedData = WeakPropertyValue.Get()->GetRecordedData();

		if (RecordedData != FirstRecordedData)
		{
			bSameValue = false;
			break;
		}
	}

	UpdateComboboxStrings();

	SAssignNew(Combobox, SComboBox<TSharedPtr<FString>>)
	.OptionsSource(&EnumDisplayTexts)
	.OnGenerateWidget_Lambda([bSameValue](TSharedPtr<FString> Item)
	{
		return SNew(STextBlock).Text(FText::FromString(*Item));
	})
	.Content()
	[
		SNew(STextBlock)
		.Text(this, &FVariantManagerEnumPropertyNode::GetRecordedEnumDisplayText, bSameValue)
	]
	.OnSelectionChanged(this, &FVariantManagerEnumPropertyNode::OnComboboxSelectionChanged); // Widget updates recorded data

	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(3.0f, 0.0f, 3.0f, 0.0f))
		[
			SNew(SBox)
			.HeightOverride(21)
			.Padding(0)
			[
				Combobox.ToSharedRef()
			]
		];
}

void FVariantManagerEnumPropertyNode::OnComboboxSelectionChanged(TSharedPtr<FString> NewItem, ESelectInfo::Type SelectType)
{
	if (!Combobox.IsValid() || !NewItem.IsValid() || PropertyValues.Num() < 1)
	{
		return;
	}

	TWeakObjectPtr<UPropertyValue> PropertyValue = PropertyValues[0];
	if (!PropertyValue.IsValid())
	{
		return;
	}

	int32 ComboboxItemIndex = EnumDisplayTexts.Find(NewItem);
	if (ComboboxItemIndex == INDEX_NONE)
	{
		UE_LOGF(LogVariantManager, Warning, "FVariantManagerEnumPropertyNode::OnComboboxSelectionChanged: Invalid Combobox selection: %ls", **NewItem);
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("PropertyNodeUpdateRecordedData", "Edit captured property '{0}'"),
		FText::FromName(PropertyValue->GetPropertyName())));

	int32 EnumIndex = EnumIndices[ComboboxItemIndex];
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		if (WeakPropertyValue.IsValid())
		{
			WeakPropertyValue.Get()->SetRecordedDataFromEnumIndex(EnumIndex);
		}
	}

	GetVariantManager().Pin()->GetVariantManagerWidget()->RefreshPropertyList();
}

FText FVariantManagerEnumPropertyNode::GetRecordedEnumDisplayText(bool bSameValue) const
{
	if (!bSameValue)
	{
		return LOCTEXT("MultipleValuesLabel", "Multiple Values");
	}

	// We don't just use Combobox->GetSelectedItem() here because the Combobox watches
	// EnumDisplayTexts, and that list only contains the valid and non-hidden options for the Enum.
	// The user can only choose between those, but the PropertyValue may have a hidden or 'invalid'
	// option recorded already (e.g. _MAX), and with this we will display it as it actually is,
	// which is also how the Details panel behaves
	if (PropertyValues.Num() >= 1)
	{
		UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();

		if (UEnum* Enum = FirstPropertyValue->GetEnumPropertyEnum())
		{
			int32 EnumIndex = FirstPropertyValue->GetRecordedDataAsEnumIndex();
			FText EnumDisplayNameText = Enum->GetDisplayNameTextByIndex(EnumIndex);
			if(!EnumDisplayNameText.IsEmpty())
			{
				return EnumDisplayNameText;
			}
		}
	}

	return FText();
}

void FVariantManagerEnumPropertyNode::UpdateComboboxStrings()
{
	if (PropertyValues.Num() < 1)
	{
		return;
	}

	TWeakObjectPtr<UPropertyValue> WeakProp = PropertyValues[0];
	if (!WeakProp.IsValid())
	{
		return;
	}

	UPropertyValue* PropertyValue = WeakProp.Get();
	UEnum* Enum = PropertyValue->GetEnumPropertyEnum();
	if(!Enum)
	{
		return;
	}

	TArray<FName> AllowedPropertyEnums = PropertyValue->GetValidEnumsFromPropertyOverride();

	// Get enum doc link (not just GetDocumentationLink as that is the documentation for the struct we're in, not the enum documentation)
	FString DocLink = PropertyValue->GetEnumDocumentationLink();

	EnumDisplayTexts.Empty();
	EnumRichToolTips.Empty();
	EnumIndices.Empty();

	//													avoid _MAX
	for(int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
	{
		bool bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), EnumIndex) || Enum->HasMetaData(TEXT("Spacer"), EnumIndex);
		if(!bShouldBeHidden && AllowedPropertyEnums.Num() != 0)
		{
			bShouldBeHidden = AllowedPropertyEnums.Find(Enum->GetNameByIndex(EnumIndex)) == INDEX_NONE;
		}

		if( !bShouldBeHidden )
		{
			// See if we specified an alternate name for this value using metadata
			FString EnumName = Enum->GetNameStringByIndex(EnumIndex);
			FString EnumDisplayName = Enum->GetDisplayNameTextByIndex(EnumIndex).ToString();

			if (EnumDisplayName.Len() == 0)
			{
				EnumDisplayName = MoveTemp(EnumName);
			}

			EnumIndices.Add(EnumIndex);

			TSharedPtr<FString> EnumStr(new FString(EnumDisplayName));
			EnumDisplayTexts.Add(EnumStr);

			FText EnumValueToolTip = Enum->GetToolTipTextByIndex(EnumIndex);
			EnumRichToolTips.Add(IDocumentation::Get()->CreateToolTip(MoveTemp(EnumValueToolTip), nullptr, DocLink, MoveTemp(EnumName)));
		}
	}
}

#undef LOCTEXT_NAMESPACE
