// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"

#include "Widgets/Input/SComboBox.h"

class SToolTip;

class FVariantManagerEnumPropertyNode : public FVariantManagerPropertyNode
{
public:
	FVariantManagerEnumPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);

protected:
	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

private:
	void OnComboboxSelectionChanged(TSharedPtr<FString> NewItem, ESelectInfo::Type SelectType);
	FText GetRecordedEnumDisplayText(bool bSameValue) const;
	void UpdateComboboxStrings();

	TSharedPtr<SComboBox<TSharedPtr<FString>>> Combobox;

	TArray<TSharedPtr<FString>> EnumDisplayTexts;
	TArray<TSharedPtr<SToolTip>> EnumRichToolTips;
	TArray<int32> EnumIndices;
};
