// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerFunctionPropertyNode.h"

#include "ConsoleVariables.h"
#include "PropertyValue.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "K2Node_FunctionEntry.h"
#include "PropertyCustomizationHelpers.h"
#include "LevelVariantSets.h"
#include "VariantManager.h"
#include "VariantObjectBinding.h"
#include "BlueprintEditorModule.h"
#include "FunctionCaller.h"
#include "GraphEditorSettings.h"
#include "VariantManagerEditorCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "VariantManagerStyle.h"
#include "Dialogs/SEditFunctionCallerArgumentsDialog.h"
#include "EdGraph/EdGraph.h"
#include "Net/ReplayPlaylistTracker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "VariantManagerFunctionPropertyNode"

FVariantManagerFunctionPropertyNode::FVariantManagerFunctionPropertyNode(TWeakObjectPtr<UVariantObjectBinding> InObjectBinding, FFunctionCaller& InCaller, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(TArray<UPropertyValue*>(), InVariantManager)
	, ObjectBinding(InObjectBinding)
	, FunctionCaller(InCaller)
{
}

EVariantManagerNodeType FVariantManagerFunctionPropertyNode::GetType() const
{
	return EVariantManagerNodeType::Function;
}

void FVariantManagerFunctionPropertyNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("FunctionCaller"), LOCTEXT("FunctionCallerText", "Function Caller"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().CallFunction);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RemoveFunction);
	
	if (UE::VariantManager::Get_CVar_VariantManager_FunctionArgumentsUI().GetValueOnGameThread())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CallerArguments_Text",    "Edit Arguments..."),
			LOCTEXT("CallerArguments_Tooltip", "Modifies arguments that are to be passed to function caller"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icon.Details"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FVariantManagerFunctionPropertyNode::OnEditArguments),
				FCanExecuteAction::CreateLambda([this]() { return FunctionCaller.CanHaveArguments(); })
			)
		);
	}

	MenuBuilder.EndSection();
}

FText FVariantManagerFunctionPropertyNode::GetDisplayName() const
{
	return LOCTEXT("NodeDisplayName", "Function caller");
}

FText FVariantManagerFunctionPropertyNode::GetDisplayNameToolTipText() const
{
	return LOCTEXT("NodeToolTipText", "Reference to one of the functions of the parent LevelVariantSets' FunctionDirector");
}

TWeakObjectPtr<UVariantObjectBinding> FVariantManagerFunctionPropertyNode::GetObjectBinding() const
{
	return ObjectBinding;
}

FFunctionCaller& FVariantManagerFunctionPropertyNode::GetFunctionCaller() const
{
	return FunctionCaller;
}

uint32 FVariantManagerFunctionPropertyNode::GetDisplayOrder() const
{
	return FunctionCaller.GetDisplayOrder();
}

void FVariantManagerFunctionPropertyNode::SetDisplayOrder(uint32 InDisplayOrder)
{
	FunctionCaller.SetDisplayOrder(InDisplayOrder);
}

void FVariantManagerFunctionPropertyNode::SetBindingTargetFunction(UK2Node_FunctionEntry* NewFunctionEntry)
{
	FScopedTransaction Transaction(LOCTEXT("SetFunctionCallerFunction", "Set a new function to be called by a FunctionCaller"));

	if (ObjectBinding.IsValid())
	{
		ObjectBinding->Modify();
	}

	FunctionCaller.SetFunctionEntry(NewFunctionEntry);
}

void FVariantManagerFunctionPropertyNode::CreateDirectorFunction(ULevelVariantSets* InLevelVariantSets, UClass* PinClassType)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = GetVariantManager().Pin();
	if (PinnedVariantManager.IsValid())
	{
		const bool bCreateWithArguments = UE::VariantManager::Get_CVar_VariantManager_FunctionArgumentsUI().GetValueOnAnyThread();

		UK2Node_FunctionEntry* NewFunctionEntry = PinnedVariantManager->CreateDirectorFunction(InLevelVariantSets, FName(), PinClassType, bCreateWithArguments);
		SetBindingTargetFunction(NewFunctionEntry);

		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewFunctionEntry, false);
	}
}

void FVariantManagerFunctionPropertyNode::CreateDirectorFunctionFromFunction(UFunction* QuickBindFunction, UClass* PinClassType)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = GetVariantManager().Pin();
	if (PinnedVariantManager.IsValid())
	{
		const bool bCreateWithArguments = UE::VariantManager::Get_CVar_VariantManager_FunctionArgumentsUI().GetValueOnAnyThread();

		UK2Node_FunctionEntry* NewFunctionEntry = PinnedVariantManager->CreateDirectorFunctionFromFunction(QuickBindFunction, PinClassType, bCreateWithArguments);
		SetBindingTargetFunction(NewFunctionEntry);

		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewFunctionEntry, false);
	}
}

void FVariantManagerFunctionPropertyNode::NavigateToDefinition()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = GetVariantManager().Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	if (ULevelVariantSets* LVS = PinnedVariantManager->GetCurrentLevelVariantSets())
	{
		if (UK2Node_FunctionEntry* EntryPoint = FunctionCaller.GetFunctionEntry())
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(EntryPoint, false);
		}
		else
		{
			// Copy the body of BringKismetToFocusAttentionOnObject without the JumpToHyperlink as it would throw a warning
			TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(LVS->GetDirectorGeneratedBlueprint(), true);
			if (BlueprintEditor.IsValid())
			{
				BlueprintEditor->FocusWindow();
			}
		}
	}
}

TSharedRef<SWidget> FVariantManagerFunctionPropertyNode::GetMenuContent()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = GetVariantManager().Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	ULevelVariantSets* LVS = PinnedVariantManager->GetCurrentLevelVariantSets();

	if (LVS == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	UVariantObjectBinding* Binding = GetObjectBinding().Get();
	UClass* BoundObjectClass = nullptr;
	if (Binding)
	{
		UObject* BoundObject = GetValid(Binding->GetObject());
		if (BoundObject && !BoundObject->IsUnreachable())
		{
			BoundObjectClass = BoundObject->GetClass();
		}
	}

	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

	if (BoundObjectClass)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateNewFunctionText",    "Create New Function"),
			LOCTEXT("CreateNewFunctionTooltip", "Creates a new function in this FunctionDirector."),
			FSlateIcon(FVariantManagerStyle::GetAppStyleSetName(), "VariantManager.AddFunction"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FVariantManagerFunctionPropertyNode::CreateDirectorFunction, LVS, BoundObjectClass)
			)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("CreateQuickBinding_Text",    "Create Quick Binding"),
			LOCTEXT("CreateQuickBinding_Tooltip", "Shows a list of functions on this object binding that can be bound directly to this event"),
			FNewMenuDelegate::CreateSP(this, &FVariantManagerFunctionPropertyNode::PopulateQuickBindSubMenu, BoundObjectClass),
			false /* bInOpenSubMenuOnClick */,
			FSlateIcon(FVariantManagerStyle::GetAppStyleSetName(), "VariantManager.AddBinding"),
			true /* bInShouldCloseWindowAfterMenuSelection */
		);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearCaller_Text",    "Clear"),
		LOCTEXT("ClearCaller_Tooltip", "Unbinds the current function from this function caller"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FVariantManagerFunctionPropertyNode::SetBindingTargetFunction, (UK2Node_FunctionEntry*)nullptr)
		)
	);

	UBlueprint* DirectorBP = GetValid(Cast<UBlueprint>(LVS->GetDirectorGeneratedBlueprint()));
	if (BoundObjectClass && DirectorBP && !DirectorBP->IsUnreachable())
	{
		FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Function_16x");

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ExistingFunctionCallers", "Existing functions"));

		UK2Node_FunctionEntry* SelectedFunction = FunctionCaller.GetFunctionEntry();
		TArray<UK2Node_FunctionEntry*> EntryNodes;
		for (UEdGraph* FunctionGraph : DirectorBP->FunctionGraphs)
		{
			EntryNodes.Reset();
			FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);

			bool bRightTargetType = false;
			FText TooltipText;
			UK2Node_FunctionEntry* ThisFunc = nullptr;

			if (EntryNodes.Num() == 1)
			{
				ThisFunc = EntryNodes[0];

				if (FunctionCaller.IsValidFunction(ThisFunc))
				{
					if (ThisFunc->UserDefinedPins.Num() == 0)
					{
						bRightTargetType = true;
						TooltipText = LOCTEXT("ValidNoInputTooltip", "Valid function with no input pins");
					}
					else
					{
						TSharedPtr<FUserPinInfo>& PinInfo = ThisFunc->UserDefinedPins[0];
						FEdGraphPinType& Type = PinInfo->PinType;
						UClass* PinClass = Cast<UClass>(Type.PinSubCategoryObject.Get());

						if (PinClass)
						{
							// See if the target pin is of the right class
							if (BoundObjectClass->IsChildOf(PinClass))
							{
								bRightTargetType = true;
								TooltipText = ThisFunc->GetTooltipText();
							}
							else
							{
								TooltipText = FText::Format(
									LOCTEXT("WrongPinTypeTooltipWithClass", "Input pin '{0}' of type '{1}' must be a reference\nto an object of a class parent to '{2}'"),
									FText::FromName(PinInfo->PinName),
									FText::FromString(PinClass->GetName()),
									FText::FromString(BoundObjectClass->GetName()));
							}
						}
						else
						{
							TooltipText = FText::Format(
								LOCTEXT("WrongPinTypeTooltip", "Input pin '{0}' is of an invalid class type!"),
								FText::FromName(PinInfo->PinName));
						}
					}
				}
				else
				{
					if (UE::VariantManager::Get_CVar_VariantManager_FunctionArgumentsUI().GetValueOnAnyThread())
					{
						TooltipText = LOCTEXT("InvalidTooltipFunctionArguments", "Function must be valid and have either: \n - Zero input pins;\n - One input pin (with type matching the bound actor's class);\n - Two input pins (with types: the bound actor's class, arguments as map of Name to String);\n - 4 input pins (with types: the bound actor's class, Level Variant Sets, Variant Set and Variant)\n - 5 input pins (with types: the bound actor's class, Level Variant Sets, Variant Set, Variant and arguments as map of Name to String)");
					}
					else
					{
						TooltipText = LOCTEXT("InvalidTooltip", "Function must be valid and have either: \n - Zero input pins;\n - One input pin (with type matching the bound actor's class);\n - 4 input pins (with types: the bound actor's class, Level Variant Sets, Variant Set and Variant)");
					}
				}
			}
			else
			{
				TooltipText = LOCTEXT("InvalidEntryNodeTooltip", "Cannot call a function with more or less than one entry node!");
			}

			MenuBuilder.AddMenuEntry(
				FunctionCaller.CanHaveArguments(ThisFunc) ?
					FText::FormatOrdered(LOCTEXT("FunctionWithArguments", "{0} (...)"), FText::FromString(FunctionGraph->GetName())) : 
					FText::FromString(FunctionGraph->GetName()),
				TooltipText,
				Icon,
				FUIAction(
					FExecuteAction::CreateSP(this, &FVariantManagerFunctionPropertyNode::SetBindingTargetFunction, ThisFunc),
					FCanExecuteAction::CreateLambda([bRightTargetType]{ return bRightTargetType; }),
					FIsActionChecked::CreateLambda([SelectedFunction, ThisFunc]()
					{
						return ThisFunc == SelectedFunction;
					})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void FVariantManagerFunctionPropertyNode::PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder, UClass* TemplateClass)
{
	FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Function_16x");

	TArray<UFunction*> Functions;

	// Constrain the menu with this box, or else it will expand way outside the screen
	TSharedRef<SBox> Box = SNew(SBox)
		.HeightOverride(500.0f)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f));

	FMenuBuilder ListMenuBuilder = FMenuBuilder(true, nullptr, nullptr, false);

	UClass* SuperClass = TemplateClass;
	while (SuperClass)
	{
		ListMenuBuilder.BeginSection(NAME_None, SuperClass->GetDisplayNameText());

		Functions.Reset();
		for (UFunction* Function : TFieldRange<UFunction>(SuperClass, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated))
		{
			if (Function->HasAllFunctionFlags(FUNC_BlueprintCallable|FUNC_Public))
			{
				Functions.Add(Function);
			}
		}

		Algo::SortBy(Functions, &UFunction::GetFName, FNameLexicalLess());

		for (UFunction* Function : Functions)
		{
			ListMenuBuilder.AddMenuEntry(
				FText::FromName(Function->GetFName()),
				FText(),
				Icon,
				FUIAction(
					FExecuteAction::CreateSP(this, &FVariantManagerFunctionPropertyNode::CreateDirectorFunctionFromFunction, Function, TemplateClass)
				)
			);
		}

		ListMenuBuilder.EndSection();

		SuperClass = SuperClass->GetSuperClass();
	}

	// @hack to make the inner ListMenuBuilder widget have a transparent border, making
	// the dialog look a bit nicer. This is necessary while we have no control of the content
	// padding of the widgets generated by FMenuBuilder. Without this, we would see an outer
	// rectangle (widget generated by the outer MenuBuilder) with a content padding that
	// we can't get away from, and an inner rectangle a few pixels in with our SBox
	TSharedRef<SWidget> ResultWidget = ListMenuBuilder.MakeWidget();
	FName Type = ResultWidget->GetType();
	if (Type == FName(TEXT("SMultiBoxWidget")))
	{
		TSharedRef<SMultiBoxWidget> MultiBox = StaticCastSharedRef<SMultiBoxWidget>(ResultWidget);
		FChildren* Children = MultiBox->GetChildren();
		for (int32 Index = 0; Index < Children->Num(); Index++)
		{
			TSharedRef<SWidget> ChildWidget = Children->GetChildAt(Index);
			FName ChildType = ChildWidget->GetType();

			if (ChildType == FName(TEXT("SBorder")))
			{
				TSharedRef<SBorder> BorderChild = StaticCastSharedRef<SBorder>(ChildWidget);
				BorderChild->SetBorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
			}
		}
	}

	Box->SetContent(ResultWidget);
	MenuBuilder.AddWidget(Box, FText(), true);
}

const FSlateBrush* FVariantManagerFunctionPropertyNode::GetFunctionIcon() const
{
	if (FunctionCaller.IsValidFunction(FunctionCaller.GetFunctionEntry()))
	{
		return FAppStyle::GetBrush("GraphEditor.Function_16x");
	}

	return FAppStyle::GetBrush("Sequencer.UnboundEvent");
}

FReply FVariantManagerFunctionPropertyNode::ResetMultipleValuesToDefault()
{
	SetBindingTargetFunction(nullptr);
	return FReply::Handled();
}

bool FVariantManagerFunctionPropertyNode::PropertiesHaveSameValue() const
{
	return true;
}

bool FVariantManagerFunctionPropertyNode::PropertiesHaveDefaultValue() const
{
	return FunctionCaller.GetFunctionEntry() == nullptr;
}

TSharedPtr<SWidget> FVariantManagerFunctionPropertyNode::GetPropertyValueWidget()
{
	const bool bFunctionArgumentsUI = UE::VariantManager::Get_CVar_VariantManager_FunctionArgumentsUI().GetValueOnAnyThread();

	// Make the button and set its foreground color to white so that it doesn't turn black when we have the
	// row selected
	TSharedRef<SWidget> BrowseButton = bFunctionArgumentsUI ? 
		PropertyCustomizationHelpers::MakeEditButton(FSimpleDelegate::CreateSP(this, &FVariantManagerFunctionPropertyNode::NavigateToDefinition), LOCTEXT("EditFunction_Tip", "Edit this function")) : 
		PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FVariantManagerFunctionPropertyNode::NavigateToDefinition), LOCTEXT("NavigateToDefinition_Tip", "Navigate to this function's definition"));
	TSharedRef<SButton> BrowseButtonAsButton = StaticCastSharedRef<SButton>(BrowseButton);
	BrowseButtonAsButton->SetForegroundColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));

	TSharedPtr<SHorizontalBox> HorizontalBox;


	// Taken from FMovieSceneEventCustomization::CustomizeChildren
	TSharedPtr<SWidget> Box = SNew(SBox)
	.MinDesiredWidth(200.f)
	.Padding(FMargin(4.0f, 3.0f, 0.0f, 2.0f))
	[
		SAssignNew(HorizontalBox, SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SComboButton)
			.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
			.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.OnGetMenuContent(this, &FVariantManagerFunctionPropertyNode::GetMenuContent)
			.CollapseMenuOnParentFocus(true)
			.ContentPadding(FMargin(4.f, 0.f))
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &FVariantManagerFunctionPropertyNode::GetFunctionIcon)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("PropertyEditor.AssetClass"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text_Lambda([this]()
					{
						return FText::FromName(FunctionCaller.FunctionName);
					})
				]
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(7.0f, 0.0f, 6.0f, 0.0f))
		[
			BrowseButtonAsButton
		]
	];

	if (bFunctionArgumentsUI)
	{
		if (this->FunctionCaller.CanHaveArguments())
		{
			if (const UGraphEditorSettings* EditorSettings = GetDefault<UGraphEditorSettings>())
			{
				HorizontalBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(7.0f, 0.0f, 6.0f, 0.0f))
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(22.0f)
					.HeightOverride(22.0f)
					.ToolTipText(this, &FVariantManagerFunctionPropertyNode::GetFunctionArgumentsTooltipText)
					[
						SNew(SButton)
						.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
						.OnClicked_Lambda([this]()
						{
							OnEditArguments();
							return FReply::Handled();
						})
						.ContentPadding(0.0f)
						.IsFocusable(true)
						[ 
							SNew(SLayeredImage, 
								TAttribute<const FSlateBrush*>(FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapValueTypeIcon"))),
								FSlateColor::UseSubduedForeground())
							.Image(FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapKeyTypeIcon")))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				];
			}
		}
	}

	return Box;
}

void FVariantManagerFunctionPropertyNode::OnEditArguments()
{
	TSharedPtr<SEditFunctionCallerArgumentsDialog> Dialog = 
		SEditFunctionCallerArgumentsDialog::OpenDialogAsModalWindow(
			FunctionCaller.FunctionName,
			FunctionCaller.FunctionArguments);

	if (Dialog->GetUserAccepted())
	{
		const TMap<FName, FString> FunctionArguments = Dialog->GetFunctionArguments();

		if (TStrongObjectPtr<UVariantObjectBinding> VariantObjectBinding = GetObjectBinding().Pin())
		{
			TArray<FFunctionCaller>& FunctionCallers = VariantObjectBinding->GetFunctionCallers();

			if (FFunctionCaller* Found = FunctionCallers.FindByPredicate([FunctionName=FunctionCaller.FunctionName](const FFunctionCaller& Element)
				{
					return Element.FunctionName == FunctionName;
				}))
			{
				FScopedTransaction EditFunctionArguments(
					*FunctionCaller.FunctionName.ToString(),
					LOCTEXT("EditFunctionCallerArgumentsTransaction", "Edit Function Caller Arguments"),
						VariantObjectBinding.Get());

				VariantObjectBinding->Modify();
				Found->FunctionArguments = FunctionArguments;
			}
		}
	}
}

FText FVariantManagerFunctionPropertyNode::GetFunctionArgumentsTooltipText() const
{
	if (FunctionCaller.FunctionArguments.Num() == 0)
	{
		return LOCTEXT("NoFunctionArguments", "No function arguments specified.");
	}

	FTextBuilder TextBuilder;
	TextBuilder.AppendLineFormat(LOCTEXT("FunctionArguments", "{0} function arguments:"),
		FFormatOrderedArguments({ FText::AsNumber(FunctionCaller.FunctionArguments.Num()) }));
	
	for (const TTuple<FName, FString>& Pair : FunctionCaller.FunctionArguments)
	{
		TextBuilder.AppendLineFormat(LOCTEXT("FunctionArgumentsPairFmt", "{0} = {1}"), 
			FFormatOrderedArguments({ FText::FromName(Pair.Key), FText::FromString(Pair.Value) }));
	}

	return TextBuilder.ToText();
}

#undef LOCTEXT_NAMESPACE
