// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"

class UK2Node_FunctionEntry;
enum class EVariantManagerNodeType : uint8;
struct FFunctionCaller;

class SVariantManagerTableRow;
class UVariantObjectBinding;
class ULevelVariantSets;
class FReply;

class FVariantManagerFunctionPropertyNode
	: public FVariantManagerPropertyNode
{
public:

	FVariantManagerFunctionPropertyNode(TWeakObjectPtr<UVariantObjectBinding> InObjectBinding, FFunctionCaller& InCaller, TWeakPtr<FVariantManager> InVariantManager);

	// FVariantManagerDisplayNode interface
	virtual EVariantManagerNodeType GetType() const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayNameToolTipText() const override;
	//~ End FVariantManagerDisplayNode interface

	TWeakObjectPtr<UVariantObjectBinding> GetObjectBinding() const;
	FFunctionCaller& GetFunctionCaller() const;

	/**
	* Get the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	*/
	virtual uint32 GetDisplayOrder() const override;

	/**
	* Set the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	*/
	virtual void SetDisplayOrder(uint32 InDisplayOrder) override;

private:

	// Set the target function as the selected option both for this node as well as for the underlying binding
	void SetBindingTargetFunction(UK2Node_FunctionEntry* NewFunctionEntry);

	// Create a new blank director function
	void CreateDirectorFunction(ULevelVariantSets* InLevelVariantSets, UClass* PinClassType);

	void CreateDirectorFunctionFromFunction(UFunction* QuickBindFunction, UClass* PinClassType);

	// Navigate to the function whose name is FunctionName
	void NavigateToDefinition();

	// Generate the drop down menu
	TSharedRef<SWidget> GetMenuContent();

	// Populate the menu builder with entries according to UClass' functions
	void PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder, UClass* TemplateClass);

	// Get the icon to display on the main combo button
	const FSlateBrush* GetFunctionIcon() const;

protected:

	// Reset recorded data of all property values to nullptr
	virtual FReply ResetMultipleValuesToDefault() override;

	// Disable this function as we don't have any properties to record with
	virtual FReply RecordMultipleValues() override { return FReply::Handled(); };

	// Returns true if all properties have the exact same value bytes
	virtual bool PropertiesHaveSameValue() const override;

	// Returns true if all properties have the exact same value bytes as the CDO
	virtual bool PropertiesHaveDefaultValue() const override;

	// Disable this function as we don't have any properties
	virtual bool PropertiesHaveCurrentValue() const override { return true; };

	// Generate the widget for the entire row
	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

	// Opens Edit Arguments dialog
	void OnEditArguments();

	// Formats supplied function arguments into tooltip text
	FText GetFunctionArgumentsTooltipText() const;

	TWeakObjectPtr<UVariantObjectBinding> ObjectBinding;
	FFunctionCaller& FunctionCaller;
};
