// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerOptionPropertyNode.h"

#include "Framework/Views/TableViewMetadata.h"
#include "PropertyValue.h"
#include "SwitchActor.h"
#include "VariantManagerLog.h"
#include "VariantObjectBinding.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FVariantManagerOptionPropertyNode"

FVariantManagerOptionPropertyNode::FVariantManagerOptionPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(InPropertyValues, InVariantManager)
{
}

FText FVariantManagerOptionPropertyNode::GetDisplayNameToolTipText() const
{
	return LOCTEXT("SelectedOptionTooltip", "Which child actor is visible");
}

TSharedPtr<SWidget> FVariantManagerOptionPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() != 1)
	{
		return SNew(SBox);
	}

	TWeakObjectPtr<UPropertyValue> PropValPtr = PropertyValues[0];
	if (!PropValPtr.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "PropertyValue was invalid!");
		return SNew(SBox);
	}

	ASwitchActor* SwitchActor = GetSwitchActor();
	if (!SwitchActor)
	{
		return SNew(SBox);
	}

	TArray<AActor*> ActorOptions = SwitchActor->GetOptions();

	UPropertyValue* PropVal = PropertyValues[0].Get();
	if (PropVal->Resolve())
	{
		if (!PropVal->HasRecordedData())
		{
			PropVal->RecordDataFromResolvedObject();
		}
	}
	else
	{
		return GetFailedToResolveWidget(PropVal);
	}

	Options.Reset();
	Algo::Transform(ActorOptions, Options, [](const AActor* Actor)
	{
		return MakeShared<FString>(Actor->GetActorLabel());
	});

	// Prepare Options and initially selected item
	// Note that we can't use SetSelectedItem explicitly or else that would
	// trigger OnComboBoxOptionChanged, which would SetRecordedData and
	// might trigger the property widget to be rebuilt
	TSharedPtr<FString>* InitiallySelected = nullptr;
	int32 SelectedIndex = *((int32*)PropVal->GetRecordedData().GetData());
	if (Options.IsValidIndex(SelectedIndex))
	{
		InitiallySelected = &Options[SelectedIndex];
	}

	SAssignNew(ComboBox, SComboBox<TSharedPtr<FString>>)
	.OptionsSource(&Options)
	.InitiallySelectedItem(InitiallySelected? *InitiallySelected : nullptr)
	.OnComboBoxOpening(this, &FVariantManagerOptionPropertyNode::OnComboBoxOpening)
	.OnSelectionChanged(this, &FVariantManagerOptionPropertyNode::OnComboBoxOptionChanged)
	.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
	{
		return SNew(STextBlock).Text(Item.IsValid() ? FText::FromString(*Item) : FText::GetEmpty());
	})
	.Content()
	[
		SNew(STextBlock).Text(this, &FVariantManagerOptionPropertyNode::GetComboBoxSelectedOptionText)
	];

	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(3.0f, 0.0f, 3.0f, 0.0f))
		[
			SNew(SBox)
			.HeightOverride(21)
			.Padding(0)
			[
				ComboBox.ToSharedRef()
			]
		];
}

ASwitchActor* FVariantManagerOptionPropertyNode::GetSwitchActor()
{
	if (PropertyValues.Num() != 1)
	{
		return nullptr;
	}

	TWeakObjectPtr<UPropertyValue> PropValPtr = PropertyValues[0];
	if (!PropValPtr.IsValid())
	{
		return nullptr;
	}

	UVariantObjectBinding* Binding = PropValPtr->GetParent();
	if (!Binding)
	{
		return nullptr;
	}

	if (ASwitchActor* Actor = Cast<ASwitchActor>(Binding->GetObject()))
	{
		return Actor;
	}

	return nullptr;
}

void FVariantManagerOptionPropertyNode::OnComboBoxOpening()
{
	ASwitchActor* SwitchActor = GetSwitchActor();
	if (SwitchActor == nullptr)
	{
		return;
	}

	TArray<AActor*> Actors = SwitchActor->GetOptions();
	TSharedPtr<FString> OldSelection = ComboBox->GetSelectedItem();

	Options.Reset();
	Algo::Transform(Actors, Options, [](const AActor* Actor)
	{
		return MakeShared<FString>(Actor->GetActorLabel());
	});

	ComboBox->ClearSelection();
	ComboBox->RefreshOptions();

	// Try restoring the old selection (so that it shows as the selected item on the
	// combobox while it is expanded)
	if (OldSelection.IsValid())
	{
		FString OldStr = *OldSelection;
		TSharedPtr<FString>* NewSelection = Options.FindByPredicate([&OldStr](const TSharedPtr<FString>& Item)
		{
			return Item.IsValid() && *Item == OldStr;
		});
		ComboBox->SetSelectedItem(*NewSelection);
	}
}

void FVariantManagerOptionPropertyNode::OnComboBoxOptionChanged(TSharedPtr<FString> NewOption, ESelectInfo::Type SelectType)
{
	if (PropertyValues.Num() != 1 || !NewOption.IsValid())
	{
		return;
	}

	TWeakObjectPtr<UPropertyValue> PropValPtr = PropertyValues[0];
	if (!PropValPtr.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("OptionChangedTransaction", "Edit captured property '{0}' with option '{1}'"),
		FText::FromName(PropValPtr->GetPropertyName()),
		FText::FromString(NewOption.IsValid()? *NewOption : TEXT("None")))
	);

	int32 Index = Options.Find(NewOption);
	PropValPtr->SetRecordedData((uint8*)&Index, sizeof(int32));
}

FText FVariantManagerOptionPropertyNode::GetComboBoxSelectedOptionText() const
{
	TSharedPtr<FString> SelectedItem = ComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
