// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"
#include "Widgets/Input/SComboBox.h"

class ASwitchActor;

// Display node made to interact with SwitchActor and UPropertyValueOption
class FVariantManagerOptionPropertyNode : public FVariantManagerPropertyNode
{
public:
	FVariantManagerOptionPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);
	virtual FText GetDisplayNameToolTipText() const override;

protected:
	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

private:
	ASwitchActor* GetSwitchActor();

	void OnComboBoxOpening();
	void OnComboBoxOptionChanged(TSharedPtr<FString> NewOption, ESelectInfo::Type SelectType);
	FText GetComboBoxSelectedOptionText() const;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBox;
	TArray<TSharedPtr<FString>> Options;
};
