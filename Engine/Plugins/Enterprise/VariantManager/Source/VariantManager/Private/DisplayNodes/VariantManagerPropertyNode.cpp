// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerPropertyNode.h"

#include "PropertyTemplateObject.h"
#include "PropertyValue.h"
#include "SVariantManager.h"
#include "UObject/Package.h"
#include "VariantManager.h"
#include "VariantManagerDragDropOp.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerLog.h"
#include "VariantObjectBinding.h"

#include "EditorFontGlyphs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableRow.h"
#define LOCTEXT_NAMESPACE "FVariantManagerPropertyNode"

using FDisplayNodeRef = TSharedRef<FVariantManagerDisplayNode>;

FVariantManagerPropertyNode::FVariantManagerPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerDisplayNode(nullptr, nullptr)
	, PropertyValues(InPropertyValues)
	, VariantManager(InVariantManager)
{
}

void FVariantManagerPropertyNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FVariantManagerDisplayNode::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection(TEXT("Captured property"), LOCTEXT("CapturedPropertyText", "Captured property"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().ApplyProperty);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RecordProperty);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().RemoveCapture);
	MenuBuilder.EndSection();
}

FText FVariantManagerPropertyNode::GetDisplayNameToolTipText() const
{
	if (PropertyValues.Num() < 1 || !PropertyValues[0].IsValid())
	{
		return FText();
	}

	return PropertyValues[0].Get()->GetPropertyTooltip();
}

const FSlateBrush* FVariantManagerPropertyNode::GetIconOverlayBrush() const
{
	return nullptr;
}

FText FVariantManagerPropertyNode::GetIconToolTipText() const
{
	return FText();
}

EVariantManagerNodeType FVariantManagerPropertyNode::GetType() const
{
	return EVariantManagerNodeType::Property;
}

bool FVariantManagerPropertyNode::IsReadOnly() const
{
	return true;
}

FText FVariantManagerPropertyNode::GetDisplayName() const
{
	if (PropertyValues.Num() < 1 || !PropertyValues[0].IsValid())
	{
		return FText();
	}

	return FText::FromString(PropertyValues[0].Get()->GetFullDisplayString());
}

void FVariantManagerPropertyNode::SetDisplayName(const FText& NewDisplayName)
{
	return;
}

bool FVariantManagerPropertyNode::IsSelectable() const
{
	return true;
}

TOptional<EItemDropZone> FVariantManagerPropertyNode::CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const
{
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();
	if (!VarManDragDrop.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	TArray<FDisplayNodeRef> PropOrFuncNodes;
	for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
	{
		if ((DraggedNode->GetType() == EVariantManagerNodeType::Property ||
			 DraggedNode->GetType() == EVariantManagerNodeType::Function ) &&
			DraggedNode != SharedThis(this)) // No point in reordering the node ondo itself
		{
			PropOrFuncNodes.Add(DraggedNode);
		}
	}

	if (PropOrFuncNodes.Num() > 0)
	{
		FText NewHoverText = FText::Format( LOCTEXT("CanDrop_Captures", "Reorder '{0}'"), PropOrFuncNodes[0]->GetDisplayName());

		const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

		VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

		return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
	}

	if (VarManDragDrop.IsValid())
	{
		VarManDragDrop->ResetToDefaultToolTip();
	}

	return TOptional<EItemDropZone>();
}

void FVariantManagerPropertyNode::Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone)
{
	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return;
	}

	TSharedPtr<SVariantManager> VarManWidget = VarMan->GetVariantManagerWidget();
	if (!VarManWidget.IsValid())
	{
		return;
	}

	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();
	if (!VarManDragDrop.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> DraggedPropertyNodes;
	for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
	{
		DraggedPropertyNodes.Add(StaticCastSharedRef<FVariantManagerPropertyNode>(DraggedNode));
	}

	VarManWidget->ReorderPropertyNodes(DraggedPropertyNodes, SharedThis(this), ItemDropZone);
}

// We replace the standard reset button with our own. No easy way to do this other than to dig
// around for it in the widget hierarchy
bool FVariantManagerPropertyNode::RecursiveDisableOldResetButton(TSharedPtr<SWidget> Widget)
{
	FString TypeString = Widget->GetTypeAsString();
	if (TypeString == TEXT("SResetToDefaultPropertyEditor"))
	{
		Widget->SetVisibility(EVisibility::Collapsed);
		return true;
	}

	FChildren* Children = Widget->GetChildren();
	for (int32 Index = 0; Index < Children->Num(); ++Index)
	{
		if (RecursiveDisableOldResetButton(Children->GetChildAt(Index)))
		{
			return true;
		}
	}

	return false;
}

TSharedPtr<SWidget> FVariantManagerPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no UPropertyValues!");
		return SNew(SBox);
	}

	// Check to see if we have all valid, equal UPropertyValues
	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();

	if (!FirstPropertyValue)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no first property value!");
		return SNew(SBox);
	}

	uint32 FirstPropHash = FirstPropertyValue->GetPropertyPathHash();
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (!PropertyValue.IsValid())
		{
			UE_LOGF(LogVariantManager, Error, "PropertyValue was invalid!");
			return SNew(SBox);
		}

		if (PropertyValue.Get()->GetPropertyPathHash() != FirstPropHash)
		{
			UE_LOGF(LogVariantManager, Error, "A PropertyNode's PropertyValue array describes properties with different paths!");
			return SNew(SBox);
		}
	}

	// If all properties fail to resolve, just give back a "Failed to resolve" text block
	bool bAtLeastOneResolved = false;
	bool bSomeFailedToResolve = false;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		if (WeakPropertyValue.IsValid() && WeakPropertyValue->Resolve())
		{
			if (!WeakPropertyValue->HasRecordedData())
			{
				WeakPropertyValue->RecordDataFromResolvedObject();
			}

			bAtLeastOneResolved = true;
		}
		else
		{
			bSomeFailedToResolve = true;
		}
	}
	if(!bAtLeastOneResolved)
	{
		return GetFailedToResolveWidget(FirstPropertyValue);
	}

	if (bSomeFailedToResolve)
	{
		UE_LOGF(LogVariantManager, Warning, "Some properties of capture '%ls' failed to resolve!", *GetDisplayName().ToString());
	}

	if (!PropertiesHaveSameValue())
	{
		return GetMultipleValuesWidget();
	}

	FSinglePropertyParams InitParams;
	InitParams.NamePlacement = EPropertyNamePlacement::Hidden;

	UPropertyTemplateObject* Template = NewObject<UPropertyTemplateObject>(GetTransientPackage());
	SinglePropertyViewTemplate.Reset(Template);

	FFieldClass* PropertyClass = FirstPropertyValue->GetPropertyClass();

	// Find the property responsible for Template's UObjectProperty
	FObjectPropertyBase* TemplateObjectProp = nullptr;
	if (PropertyClass && PropertyClass->IsChildOf(FObjectPropertyBase::StaticClass()))
	{
		for (TFieldIterator<FObjectPropertyBase> PropertyIterator(Template->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			FObjectPropertyBase* ObjectProp = *PropertyIterator;
			if (ObjectProp->GetClass() == PropertyClass)
			{
				TemplateObjectProp = ObjectProp;
				break;
			}
		}
	}

	// HACK to cause the widget to display an FObjectProperty editor restricted to objects of our desired class
	// Note that we undo this right aftewards, so that other property value widgets can do the same to different
	// classes. The template's property itself will then be free to be set with whatever object, but the created
	// widget is already locked in place
	if (TemplateObjectProp)
	{
		TemplateObjectProp->PropertyClass = FirstPropertyValue->GetObjectPropertyObjectClass();
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedPtr<ISinglePropertyView> SinglePropView = PropertyEditorModule.CreateSingleProperty(
		SinglePropertyViewTemplate.Get(),
		UPropertyTemplateObject::GetPropertyNameFromClass(PropertyClass),
		InitParams);

	// Reset it back to generic
	if (TemplateObjectProp)
	{
		TemplateObjectProp->PropertyClass = UObject::StaticClass();
	}

	if (!SinglePropView)
	{
		return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("UnsupportedPropertyType", "Unsupported property type!"))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.ColorAndOpacity(this, &FVariantManagerDisplayNode::GetDisplayNameColor)
			.ToolTipText(FText::Format(
				LOCTEXT("UnsupportedPropertyTypeTooltip", "Properties of class '{0}' can't be captured yet!"),
				FText::FromString(FirstPropertyValue->GetPropertyClass()->GetName())))
		];
	}

	RecursiveDisableOldResetButton(SinglePropView);

	TSharedPtr<IPropertyHandle> PropHandle = SinglePropView->GetPropertyHandle();

	// Update widget with recorded data
	TArray<void*> RawDataForEachObject;
	PropHandle->AccessRawData(RawDataForEachObject);
	void* SinglePropWidgetDataPtr = RawDataForEachObject.IsValidIndex(0) ? 
		RawDataForEachObject[0] :	// We'll always pass just a single object
		nullptr;					// In case nothing is passed

	const TArray<uint8>& FirstRecordedData = FirstPropertyValue->GetRecordedData();

	if (SinglePropWidgetDataPtr)
	{
		if (FSoftObjectProperty* Prop = CastField<FSoftObjectProperty>(PropHandle->GetProperty()))
		{
			if (FirstRecordedData.Num() >= sizeof(UObject*))
			{
				UObject* TargetObject = *((UObject**)(FirstRecordedData.GetData()));
				Prop->SetObjectPropertyValue(SinglePropWidgetDataPtr, TargetObject);
			}
		}
		else
		{
			if (FirstRecordedData.Num() >= FirstPropertyValue->GetValueSizeInBytes())
			{
				if (const uint8* DataPtr = FirstRecordedData.GetData())
				{
					FMemory::Memcpy((uint8*)SinglePropWidgetDataPtr, DataPtr, FirstPropertyValue->GetValueSizeInBytes());
				}
			}
		}
	}

	// Update recorded data when user modifies the widget (modifying the widget will modify the
	// property value of the object the widget is looking at e.g. the class metadata object)
	PropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FVariantManagerPropertyNode::UpdateRecordedDataFromSinglePropView, SinglePropView));

	return SinglePropView;
}

TSharedRef<SWidget> FVariantManagerPropertyNode::GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow)
{
	// Using this syncs all splitters between property nodes
	if (TSharedPtr<FVariantManager> VariantManagerPtr = GetVariantManager().Pin())
	{
		if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManagerPtr->GetVariantManagerWidget())
		{
			FVariantManagerPropertiesColumnSizeData& ColumnSizeData = VariantManagerWidget->GetPropertiesColumnSizeData();

			return SNew(SBox)
			[
				SNew(SBorder)
				.VAlign(VAlign_Center)
				.BorderImage(this, &FVariantManagerDisplayNode::GetNodeBorderImage)
				.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
				[
					SNew(SSplitter)
					.Style(FAppStyle::Get(), "DetailsView.Splitter")
					.PhysicalSplitterHandleSize(1.0f)
					.HitDetectionSplitterHandleSize(5.0f)
					.Orientation(Orient_Horizontal)

					+ SSplitter::Slot()
					.Value(ColumnSizeData.NameColumnWidth)
					.OnSlotResized(ColumnSizeData.OnSplitterNameColumnChanged)
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(FMargin( 4.0, 0.0f, 0.0f, 0.0f ))
						.HeightOverride(26) // Sum of paddings for the rows used in a details view like this
						[
							SAssignNew(EditableLabel, SInlineEditableTextBlock)
							.IsReadOnly(true)
							.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
							.ColorAndOpacity(this, &FVariantManagerDisplayNode::GetDisplayNameColor)
							.OnTextCommitted(this, &FVariantManagerDisplayNode::HandleNodeLabelTextChanged)
							.Text(this, &FVariantManagerDisplayNode::GetDisplayName)
							.ToolTipText(this, &FVariantManagerDisplayNode::GetDisplayNameToolTipText)
							.Clipping(EWidgetClipping::ClipToBounds)
						]
					]

					+ SSplitter::Slot()
					.Value(ColumnSizeData.ValueColumnWidth)
					.OnSlotResized(ColumnSizeData.OnSplitterValueColumnChanged)
					[
						GetCustomValueContent()
					]
				]
			];
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FVariantManagerPropertyNode::GetCustomValueContent()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SBox)
			.Padding(FMargin( 3.0, 0.0f, 3.0f, 0.0f ))
			[
				GetPropertyValueWidget().ToSharedRef()
			]
		]

		+SHorizontalBox::Slot()
		.Padding( FMargin(0.0f, 0.0f, 3.0f, 0.0) )
		.AutoWidth()
		.VAlign( VAlign_Center )
		[
			SNew(SBox)
			[
				SAssignNew(RecordButton, SButton)
				.IsFocusable(false)
				.ToolTipText(LOCTEXT("UseCurrentTooltip", "Record the current value for this property"))
				.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
				.ContentPadding(6.0f)
				.OnClicked(this, &FVariantManagerPropertyNode::RecordMultipleValues)
				.Visibility(FVariantManagerPropertyNode::GetRecordButtonVisibility())
				.Content()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.ShadowColorAndOpacity(FVector::ZeroVector)
					.ShadowOffset(FVector::ZeroVector)
					.Text(FEditorFontGlyphs::Download)
					.Justification(ETextJustify::Center)
					.Margin(FMargin(0.0f, 3.0f, 0.0f, 0.0f))
				]
			]
		]

		+SHorizontalBox::Slot()
		.Padding( FMargin(0.0f, 0.0f, 3.0f, 0.0) )
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			[
				SAssignNew(ResetButton, SButton)
				.IsFocusable(false)
				.ToolTipText(LOCTEXT("ResetTooltip", "Reset to the property's default value"))
				.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
				.ContentPadding(FMargin(3.0f, 8.0f))
				.OnClicked(this, &FVariantManagerPropertyNode::ResetMultipleValuesToDefault)
				.Visibility(FVariantManagerPropertyNode::GetResetButtonVisibility())
				.Content()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault") )
				]
			]
		];
}

uint32 FVariantManagerPropertyNode::GetDisplayOrder() const
{
	uint32 DisplayOrder = UINT32_MAX;
	for (const TWeakObjectPtr<UPropertyValue>& PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			DisplayOrder = FMath::Min(DisplayOrder, PropertyValue->GetDisplayOrder());
		}
	}

	return DisplayOrder;
}

void FVariantManagerPropertyNode::SetDisplayOrder(uint32 InDisplayOrder)
{
	for (const TWeakObjectPtr<UPropertyValue>& PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			PropertyValue->SetDisplayOrder(InDisplayOrder);
		}
	}
}

bool FVariantManagerPropertyNode::IsActorProperty() const
{
	for (const TWeakObjectPtr<UPropertyValue>& PropertyValue : PropertyValues)
	{
		if (TStrongObjectPtr<UPropertyValue> PropValue = PropertyValue.Pin())
		{
			return !(PropertyValue->GetPropertyParentContainerClass() && !PropertyValue->GetPropertyParentContainerClass()->IsChildOf(AActor::StaticClass()));
		}
	}

	return true;
}

FString FVariantManagerPropertyNode::GetSubPath() const
{
	FString SubPath = TEXT("Actor");

	if (GetType() == EVariantManagerNodeType::Property)
	{
		for (const TWeakObjectPtr<UPropertyValue>& PropertyValue : PropertyValues)
		{
			if (TStrongObjectPtr<UPropertyValue> PropValue = PropertyValue.Pin())
			{
				if (PropValue->GetPropertyParentContainerClass() && !PropValue->GetPropertyParentContainerClass()->IsChildOf(AActor::StaticClass()))
				{
					SubPath = PropValue->GetBaseDisplayString().TrimStartAndEnd();
				}
			}
		}
	}

	return SubPath;
}

FString FVariantManagerPropertyNode::GetPropertyName() const
{
	for (const TWeakObjectPtr<UPropertyValue>& PropertyValue : PropertyValues)
	{
		if (TStrongObjectPtr<UPropertyValue> PropValue = PropertyValue.Pin())
		{
			return PropValue->GetPropertyName().ToString();
		}
	}

	return FString();
}

void FVariantManagerPropertyNode::UpdateRecordedDataFromSinglePropView(TSharedPtr<ISinglePropertyView> SinglePropView)
{
	// Warning: This also fires after UpdateSinglePropViewFromRecordedData when that fires

	// Get the address of the data the user just input
	TSharedPtr<IPropertyHandle> PropHandle = SinglePropView->GetPropertyHandle();
	TArray<void*> RawDataForEachObject;
	PropHandle->AccessRawData(RawDataForEachObject);
	void* SinglePropWidgetDataPtr = RawDataForEachObject.IsValidIndex(0) ? 
		RawDataForEachObject[0] :	// We'll always pass just a single object
		nullptr;					// Or in case nothing is passed

	if (SinglePropWidgetDataPtr)
	{
		for (TWeakObjectPtr<UPropertyValue> PropertyValuePtr : PropertyValues)
		{
			UPropertyValue* PropertyValue = PropertyValuePtr.Get();
			if (!PropertyValue)
			{
				continue;
			}

			if (FSoftObjectProperty* Prop = CastField<FSoftObjectProperty>(PropHandle->GetProperty()))
			{
				UObject* NewObj = Prop->LoadObjectPropertyValue(SinglePropWidgetDataPtr);
				PropertyValue->SetRecordedData(reinterpret_cast<uint8*>(&NewObj), sizeof(UObject*));
			}
			else
			{
				PropertyValue->SetRecordedData(static_cast<uint8*>(SinglePropWidgetDataPtr), PropHandle->GetProperty()->GetElementSize());
			}
		}
	}

	RecordButton->SetVisibility(GetRecordButtonVisibility());
	ResetButton->SetVisibility(GetResetButtonVisibility());
}

FReply FVariantManagerPropertyNode::ResetMultipleValuesToDefault()
{
	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("ResetPropertyValue", "Reset {0} property {0}|plural(one=capture,other=captures) to default"),
		PropertyValues.Num()
	));

	TArray<uint8> DefaultValue;
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			DefaultValue = PropertyValue->GetDefaultValue();
			break;
		}
	}

	if (DefaultValue.Num() == 0)
	{
		UE_LOGF(LogVariantManager, Error, "Failed to find a valid default value for property '%ls'", *GetDisplayName().ToString());
		return FReply::Handled();
	}

	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			PropertyValue->SetRecordedData(DefaultValue.GetData(), DefaultValue.Num());
		}
	}

	if (TSharedPtr<FVariantManager> VariantManagerPtr = GetVariantManager().Pin())
	{
		if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManagerPtr->GetVariantManagerWidget())
		{
			VariantManagerWidget->RefreshPropertyList();
		}
	}

	return FReply::Handled();
}

FReply FVariantManagerPropertyNode::RecordMultipleValues()
{
	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("RecordedNewDataForProperty", "Recorded new data for {0} property {0}|plural(one=capture,other=captures)"),
		PropertyValues.Num()
	));

	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid() && PropertyValue->HasValidResolve())
		{
			PropertyValue->RecordDataFromResolvedObject();
		}
	}

	if (TSharedPtr<FVariantManager> VariantManagerPtr = GetVariantManager().Pin())
	{
		if (TSharedPtr<SVariantManager> VariantManagerWidgetPtr = VariantManagerPtr->GetVariantManagerWidget())
		{
			VariantManagerWidgetPtr->RefreshPropertyList();
		}
	}

	return FReply::Handled();
}

bool FVariantManagerPropertyNode::PropertiesHaveSameValue() const
{
	if (PropertyValues.Num() > 1 && PropertyValues[0].IsValid())
	{
		if (UPropertyValue* FirstProp = PropertyValues[0].Get())
		{
			const TArray<uint8>& FirstPropVal = FirstProp->GetRecordedData();
			for (int32 Index = 1; Index < PropertyValues.Num(); ++Index)
			{
				TWeakObjectPtr<UPropertyValue> WeakPropertyValue = PropertyValues[Index];
				if (!WeakPropertyValue.IsValid() ||
					!WeakPropertyValue->HasValidResolve() ||
					WeakPropertyValue->GetRecordedData() != FirstPropVal)
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool FVariantManagerPropertyNode::PropertiesHaveDefaultValue() const
{
	if (!PropertiesHaveSameValue())
	{
		return false;
	}

	if (PropertyValues.Num() > 0 && PropertyValues[0].IsValid())
	{
		if (UPropertyValue* FirstProp = PropertyValues[0].Get())
		{
			return FirstProp->GetDefaultValue() == FirstProp->GetRecordedData();
		}
	}

	return false;
}

bool FVariantManagerPropertyNode::PropertiesHaveCurrentValue() const
{
	for (TWeakObjectPtr<UPropertyValue> Prop : PropertyValues)
	{
		if (Prop.IsValid())
		{
			if (!Prop->IsRecordedDataCurrent())
			{
				return false;
			}
		}
	}

	return true;
}

TSharedRef<SWidget> FVariantManagerPropertyNode::GetMultipleValuesWidget()
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultipleValuesLabel", "Multiple Values"))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.ColorAndOpacity(this, &FVariantManagerDisplayNode::GetDisplayNameColor)
			.ToolTipText(LOCTEXT("MultipleValuesTooltip", "The selected actors have different values for this property"))
		];
}

TSharedRef<SWidget> FVariantManagerPropertyNode::GetFailedToResolveWidget(const UPropertyValue* Property)
{
	if (!Property)
	{
		return SNullWidget::NullWidget;
	}

	FString ActorName;

	if (UVariantObjectBinding* Binding = Property->GetParent())
	{
		if (AActor* Actor = Cast<AActor>(Binding->GetObject()))
		{
			ActorName = Actor->GetActorLabel();
		}
	}

	return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FailedToResolveText", "Failed to resolve!"))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.ColorAndOpacity(this, &FVariantManagerDisplayNode::GetDisplayNameColor)
			.ToolTipText(FText::Format(
			LOCTEXT("FailedToResolveTooltip", "Make sure actor '{0}' has a property with path '{1}'"),
			FText::FromString(ActorName), FText::FromString(Property->GetFullDisplayString())))
		];
}

EVisibility FVariantManagerPropertyNode::GetResetButtonVisibility() const
{
	bool bNoneResolved = true;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		if (WeakPropertyValue.IsValid() && WeakPropertyValue->HasValidResolve())
		{
			bNoneResolved = false;
			break;
		}
	}

	return bNoneResolved || PropertiesHaveDefaultValue() ? EVisibility::Hidden : EVisibility::Visible;
}

EVisibility FVariantManagerPropertyNode::GetRecordButtonVisibility() const
{
	bool bNoneResolved = true;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		if (WeakPropertyValue.IsValid() && WeakPropertyValue->HasValidResolve())
		{
			bNoneResolved = false;
			break;
		}
	}

	return bNoneResolved || PropertiesHaveCurrentValue() ? EVisibility::Hidden : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
