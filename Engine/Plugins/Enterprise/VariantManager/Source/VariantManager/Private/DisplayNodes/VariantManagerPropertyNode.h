// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerDisplayNode.h"
#include "UObject/StrongObjectPtr.h"

class FMenuBuilder;
struct FSlateBrush;
class ISinglePropertyView;
class UPropertyTemplateObject;
class UPropertyValue;
class SVariantManagerTableRow;
class UPropertyValue;
class FReply;
struct EVisibility;
class SButton;

class FVariantManagerPropertyNode
	: public FVariantManagerDisplayNode
{
public:

	FVariantManagerPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);

	const TArray<TWeakObjectPtr<UPropertyValue>>& GetPropertyValues() const
	{
		return PropertyValues;
	}

	// FVariantManagerDisplayNode interface
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual FText GetDisplayNameToolTipText() const override;
	virtual const FSlateBrush* GetIconOverlayBrush() const override;
	virtual FText GetIconToolTipText() const override;
	virtual EVariantManagerNodeType GetType() const override;
	virtual bool IsReadOnly() const override;
	virtual FText GetDisplayName() const override;
	virtual void SetDisplayName(const FText& NewDisplayName) override;
	virtual bool IsSelectable() const override;
	virtual bool CanDrag() const override { return true; }
	virtual TOptional<EItemDropZone> CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const override;
	virtual void Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) override;
	virtual TWeakPtr<FVariantManager> GetVariantManager() const override
	{
		return VariantManager;
	}
	//~ End FVariantManagerDisplayNode interface
	
	/**
	 * Full outliner content for this property node. This includes name, splitter and value content.
	 */
	virtual TSharedRef<SWidget> GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow) override;
	
	/**
	 * Just value content for this property node.
	 */
	virtual TSharedRef<SWidget> GetCustomValueContent();

	/**
	 * Get the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	virtual uint32 GetDisplayOrder() const;

	/**
	 * Set the order with which the VariantManager should display this in a property list. Lower values will be shown higher up
	 */
	virtual void SetDisplayOrder(uint32 InDisplayOrder);

	/**
	 * Flag if this is an Actor or Actor Component property.
	 */
	bool IsActorProperty() const;

	/**
	 * Path without the property name
	 */
	FString GetSubPath() const;

	/**
	 * Name of the property (last part of the path)
	 */
	FString GetPropertyName() const;

protected:

	// Callback for when user updates the property widget
	virtual void UpdateRecordedDataFromSinglePropView(TSharedPtr<ISinglePropertyView> SinglePropView);

	// Reset recorded data of all property values to the CDO value
	virtual FReply ResetMultipleValuesToDefault();

	// Reset recorded data from all property values
	virtual FReply RecordMultipleValues();

	// Returns true if all properties have the exact same value bytes
	virtual bool PropertiesHaveSameValue() const;

	// Returns true if all properties have the exact same value bytes as the CDO
	virtual bool PropertiesHaveDefaultValue() const;

	// Returns true if all UPropertyValues have recorded data that matches the current value of the
	// properties they are tracking
	virtual bool PropertiesHaveCurrentValue() const;

	// Returns a widget that is displayed when PropertyValues have different recorded data
	TSharedRef<SWidget> GetMultipleValuesWidget();

	// Returns a widget that is displayed when we fail to resolve PropertyValues
	TSharedRef<SWidget> GetFailedToResolveWidget(const UPropertyValue* Property);

	bool RecursiveDisableOldResetButton(TSharedPtr<SWidget> Widget);

	EVisibility GetResetButtonVisibility() const;
	EVisibility GetRecordButtonVisibility() const;

	virtual TSharedPtr<SWidget> GetPropertyValueWidget();

	TSharedPtr<SButton> ResetButton;
	TSharedPtr<SButton> RecordButton;

	TStrongObjectPtr<UPropertyTemplateObject> SinglePropertyViewTemplate;

	TMap<TWeakObjectPtr<UPropertyValue>, FDelegateHandle> PropertyValueOnRecordedSubscriptions;

	TArray<TWeakObjectPtr<UPropertyValue>> PropertyValues;

	FText DefaultDisplayName;

	TWeakPtr<FVariantManager> VariantManager;
};
