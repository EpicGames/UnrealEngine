// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerStringPropertyNode.h"

#include "PropertyValue.h"
#include "PropertyTemplateObject.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Widgets/Input/SButton.h"
#include "VariantManagerLog.h"
#include "PropertyHandle.h"
#include "ISinglePropertyView.h"
#include "UObject/TextProperty.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FVariantManagerStringPropertyNode"

FVariantManagerStringPropertyNode::FVariantManagerStringPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(InPropertyValues, InVariantManager)
{
}

TSharedPtr<SWidget> FVariantManagerStringPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no UPropertyValues!");
		return SNew(SBox);
	}

	// Check to see if we have all valid, equal UPropertyValues
	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();
	uint32 FirstPropHash = FirstPropertyValue->GetPropertyPathHash();
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (!PropertyValue.IsValid())
		{
			UE_LOGF(LogVariantManager, Error, "PropertyValue was invalid!");
			return SNew(SBox);
		}

		if (PropertyValue.Get()->GetPropertyPathHash() != FirstPropHash)
		{
			UE_LOGF(LogVariantManager, Error, "A PropertyNode's PropertyValue array describes properties with different paths!");
			return SNew(SBox);
		}
	}

	// If all properties fail to resolve, just give back a "Failed to resolve" text block
	bool bAtLeastOneResolved = false;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		UPropertyValue* PropValRaw = WeakPropertyValue.Get();
		if (PropValRaw->Resolve())
		{
			if (!PropValRaw->HasRecordedData())
			{
				PropValRaw->RecordDataFromResolvedObject();
			}

			bAtLeastOneResolved = true;
		}
	}
	if(!bAtLeastOneResolved)
	{
		return GetFailedToResolveWidget(FirstPropertyValue);
	}

	// If properties have different values, just give back a "Multiple Values" text block
	if (!PropertiesHaveSameValue())
	{
		return GetMultipleValuesWidget();
	}

	FSinglePropertyParams InitParams;
	InitParams.NamePlacement = EPropertyNamePlacement::Hidden;

	UPropertyTemplateObject* Template = NewObject<UPropertyTemplateObject>(GetTransientPackage());
	SinglePropertyViewTemplate.Reset(Template);

	// Find the property responsible for Template's UObject*
	// Assumes it has only one
	FObjectProperty* TemplateObjectProp = nullptr;
	if (FirstPropertyValue->GetPropertyClass() == FObjectProperty::StaticClass())
	{
		for (TFieldIterator<FObjectProperty> PropertyIterator(Template->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			TemplateObjectProp = *PropertyIterator;
		}
	}

	// HACK to cause the widget to display an FObjectProperty editor restricted to objects of our desired class
	// Note that we undo this right aftewards, so that other property value widgets can do the same to different
	// classes. The template's property itself will then be free to be set with whatever object, but the created
	// widget is already locked in place
	if (TemplateObjectProp)
	{
		TemplateObjectProp->PropertyClass = FirstPropertyValue->GetObjectPropertyObjectClass();
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedPtr<ISinglePropertyView> SinglePropView = PropertyEditorModule.CreateSingleProperty(
		SinglePropertyViewTemplate.Get(),
		UPropertyTemplateObject::GetPropertyNameFromClass(FirstPropertyValue->GetPropertyClass()),
		InitParams);

	if (TemplateObjectProp)
	{
		TemplateObjectProp->PropertyClass = UObject::StaticClass();
	}

	if (!SinglePropView)
	{
		return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("UnsupportedPropertyType", "Unsupported property type!"))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.ColorAndOpacity(this, &FVariantManagerDisplayNode::GetDisplayNameColor)
			.ToolTipText(FText::Format(
				LOCTEXT("UnsupportedPropertyTypeTooltip", "Properties of class '{0}' can't be captured yet!"),
				FText::FromString(FirstPropertyValue->GetPropertyClass()->GetName())))
		];
	}

	RecursiveDisableOldResetButton(SinglePropView);

	// Very important we don't transact on these SetValues, because this very function is called when Undo/Redo'ing,
	// which would put us in a loop
	TSharedPtr<IPropertyHandle> PropHandle = SinglePropView->GetPropertyHandle();
	if (FirstPropertyValue->GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()))
	{
		PropHandle->SetValue(FirstPropertyValue->GetStrPropertyString(), EPropertyValueSetFlags::NotTransactable);
	}
    else if (FirstPropertyValue->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
	{
		PropHandle->SetValue(FirstPropertyValue->GetNamePropertyName(), EPropertyValueSetFlags::NotTransactable);
	}
    else if (FirstPropertyValue->GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
	{
		PropHandle->SetValue(FirstPropertyValue->GetTextPropertyText(), EPropertyValueSetFlags::NotTransactable);
	}

	// Update recorded data when user modifies the widget (modifying the widget will modify the
	// property value of the object the widget is looking at e.g. the class metadata object)
	PropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FVariantManagerStringPropertyNode::UpdateRecordedDataFromSinglePropView, SinglePropView));

	return SinglePropView;
}

void FVariantManagerStringPropertyNode::UpdateRecordedDataFromSinglePropView(TSharedPtr<ISinglePropertyView> SinglePropView)
{
	TSharedPtr<IPropertyHandle> PropHandle = SinglePropView->GetPropertyHandle();
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			if (PropertyValue->GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()))
			{
				FString CurValue;
				PropHandle->GetValue(CurValue);
				PropertyValue.Get()->SetRecordedData((uint8*)&CurValue, sizeof(FString));
			}
			else if (PropertyValue->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
			{
				FName CurValue;
				PropHandle->GetValue(CurValue);
				PropertyValue.Get()->SetRecordedData((uint8*)&CurValue, sizeof(FName));
			}
            else if (PropertyValue->GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
			{
				FText CurValue;
				PropHandle->GetValue(CurValue);
				PropertyValue.Get()->SetRecordedData((uint8*)&CurValue, sizeof(FText));
			}
		}
	}

	RecordButton->SetVisibility(GetRecordButtonVisibility());
	ResetButton->SetVisibility(GetResetButtonVisibility());
}

#undef LOCTEXT_NAMESPACE
