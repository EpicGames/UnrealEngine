// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"

class SVariantManagerTableRow;
class FReply;

// Accesses FString, FText and FName UPropertyValues in a different
// way by actually taking copies to their objects and using SetValue on
// property handles.
// This is necessary for safety and to decouple the variant's
// stored object from the displayed object (and details view).
// This happens for these types only because they are the only base types that are
// reference types and basically handles to heap data stored elsewhere. Because of that
// we can't just blindly copy their bytes around, but *must* use the actual objects
// so that we trigger the copy constructors and serialization functions, properly caring
// for the data they handle
class FVariantManagerStringPropertyNode
	: public FVariantManagerPropertyNode
{
public:

	FVariantManagerStringPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);

protected:

	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

	// Callback for when user updates the property widget
	virtual void UpdateRecordedDataFromSinglePropView(TSharedPtr<ISinglePropertyView> SinglePropView) override;
};
