// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerStructPropertyNode.h"

#include "PropertyValue.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"
#include "VariantManagerLog.h"
#include "VariantManagerLog.h"
#include "UObject/Package.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FVariantManagerStructPropertyNode"

// Makes small, colored "X" or "R" or etc labels to place inside the numeric entry boxes
TSharedRef<SWidget> GetLabel(const FNumericProperty* InProp)
{
	FString PropName = InProp->GetName();

	static FString RName = FString(TEXT("R"));
	static FString XName = FString(TEXT("X"));
	static FString GName = FString(TEXT("G"));
	static FString YName = FString(TEXT("Y"));
	static FString BName = FString(TEXT("B"));
	static FString ZName = FString(TEXT("Z"));

	const FLinearColor* Background = &FLinearColor::Transparent;

	if (PropName == RName || PropName == XName)
	{
		Background = &SNumericEntryBox<uint8>::RedLabelBackgroundColor;
	}
	else if (PropName == GName || PropName == YName)
	{
		Background = &SNumericEntryBox<uint8>::GreenLabelBackgroundColor;
	}
	else if (PropName == BName || PropName == ZName)
	{
		Background = &SNumericEntryBox<uint8>::BlueLabelBackgroundColor;
	}

	return SNumericEntryBox<int32>::BuildNarrowColorLabel( *Background );
}

template<typename NumericType>
void ExtractNumericMetadata(FProperty* Property, TOptional<NumericType>& MinValue, TOptional<NumericType>& MaxValue, TOptional<NumericType>& SliderMinValue, TOptional<NumericType>& SliderMaxValue, NumericType& SliderExponent, NumericType& Delta, float& ShiftMultiplier, float& CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue)
{
	const FString& MetaUIMinString = Property->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = Property->GetMetaData(TEXT("UIMax"));
	const FString& SliderExponentString = Property->GetMetaData(TEXT("SliderExponent"));
	const FString& DeltaString = Property->GetMetaData(TEXT("Delta"));
	const FString& ShiftMultiplierString = Property->GetMetaData(TEXT("ShiftMultiplier"));
	const FString& CtrlMultiplierString = Property->GetMetaData(TEXT("CtrlMultiplier"));
	const FString& SupportDynamicSliderMaxValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
	const FString& SupportDynamicSliderMinValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
	const FString& ClampMinString = Property->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = Property->GetMetaData(TEXT("ClampMax"));

	// If no UIMin/Max was specified then use the clamp string
	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
	NumericType ClampMax = TNumericLimits<NumericType>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
	}

	NumericType UIMin = TNumericLimits<NumericType>::Lowest();
	NumericType UIMax = TNumericLimits<NumericType>::Max();
	TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
	TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);

	SliderExponent = NumericType(1);

	if (SliderExponentString.Len())
	{
		TTypeFromString<NumericType>::FromString(SliderExponent, *SliderExponentString);
	}

	Delta = NumericType(0);

	if (DeltaString.Len())
	{
		TTypeFromString<NumericType>::FromString(Delta, *DeltaString);
	}

	ShiftMultiplier = 10.f;
	if (ShiftMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(ShiftMultiplier, *ShiftMultiplierString);
	}

	CtrlMultiplier = 0.1f;
	if (CtrlMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(CtrlMultiplier, *CtrlMultiplierString);
	}

	const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
	const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

	MinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
	MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
	SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
	SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

	SupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
	SupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
}

#define GET_OR_EMPTY(Optional, type) Optional.IsSet()? static_cast<type>(Optional.GetValue()) : TOptional<type>()

template <typename F>
TSharedRef<SWidget> FVariantManagerStructPropertyNode::GenerateFloatEntryBox(FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset)
{
	TOptional<F> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
	F SliderExponent, Delta;
	float ShiftMultiplier = 10.f;
	float CtrlMultiplier = 0.1f;
	bool SupportDynamicSliderMaxValue = false;
	bool SupportDynamicSliderMinValue = false;

	ExtractNumericMetadata(Prop, MinValue, MaxValue, SliderMinValue, SliderMaxValue, SliderExponent, Delta, ShiftMultiplier, CtrlMultiplier, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

	return SNew(SNumericEntryBox<double>)
	.AllowSpin(true)
	.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
	.OnBeginSliderMovement(this, &FVariantManagerStructPropertyNode::OnBeginSliderMovement, Prop)
	.OnEndSliderMovement(this, &FVariantManagerStructPropertyNode::OnFloatEndSliderMovement, Prop, RecordedElementSize, Offset)
	.OnValueChanged(this, &FVariantManagerStructPropertyNode::OnFloatValueChanged, Prop)
	.OnValueCommitted(this, &FVariantManagerStructPropertyNode::OnFloatPropCommitted, Prop, RecordedElementSize, Offset)
	.Value(this, &FVariantManagerStructPropertyNode::GetFloatValueFromCache, Prop)
	.UndeterminedString(LOCTEXT("MultipleValuesLabel", "Multiple Values"))
	.ShiftMultiplier(ShiftMultiplier)
	.CtrlMultiplier(CtrlMultiplier)
	.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
	.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
	.MinValue(GET_OR_EMPTY(MinValue, double))
	.MaxValue(GET_OR_EMPTY(MaxValue, double))
	.MinSliderValue(GET_OR_EMPTY(SliderMinValue, double))
	.MaxSliderValue(GET_OR_EMPTY(SliderMaxValue, double))
	.SliderExponent(SliderExponent)
	.Delta(Delta)
	.LabelPadding( FMargin( 3 ) )
	.LabelLocation( SNumericEntryBox<double>::ELabelLocation::Inside )
	.Label()
	[
		GetLabel(Prop)
	];
}

template <typename S>
TSharedRef<SWidget> FVariantManagerStructPropertyNode::GenerateSignedEntryBox(FNumericProperty* Prop, int32 Offset)
{
	TOptional<S> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
	S SliderExponent, Delta;
	float ShiftMultiplier = 10.f;
	float CtrlMultiplier = 0.1f;
	bool SupportDynamicSliderMaxValue = false;
	bool SupportDynamicSliderMinValue = false;

	ExtractNumericMetadata(Prop, MinValue, MaxValue, SliderMinValue, SliderMaxValue, SliderExponent, Delta, ShiftMultiplier, CtrlMultiplier, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

	return SNew(SNumericEntryBox<int64>)
	.AllowSpin(true)
	.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
	.OnBeginSliderMovement(this, &FVariantManagerStructPropertyNode::OnBeginSliderMovement, Prop)
	.OnEndSliderMovement(this, &FVariantManagerStructPropertyNode::OnSignedEndSliderMovement, Prop, Offset)
	.OnValueChanged(this, &FVariantManagerStructPropertyNode::OnSignedValueChanged, Prop)
	.OnValueCommitted(this, &FVariantManagerStructPropertyNode::OnSignedPropCommitted, Prop, Offset)
	.Value(this, &FVariantManagerStructPropertyNode::GetSignedValueFromCache, Prop)
	.UndeterminedString(LOCTEXT("MultipleValuesLabel", "Multiple Values"))
	.ShiftMultiplier(ShiftMultiplier)
	.CtrlMultiplier(CtrlMultiplier)
	.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
	.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
	.MinValue(GET_OR_EMPTY(MinValue, int64))
	.MaxValue(GET_OR_EMPTY(MaxValue, int64))
	.MinSliderValue(GET_OR_EMPTY(SliderMinValue, int64))
	.MaxSliderValue(GET_OR_EMPTY(SliderMaxValue, int64))
	.SliderExponent(SliderExponent)
	.Delta(Delta)
	.LabelPadding( FMargin( 3 ) )
	.LabelLocation( SNumericEntryBox<int64>::ELabelLocation::Inside )
	.Label()
	[
		GetLabel(Prop)
	];
}

template <typename U>
TSharedRef<SWidget> FVariantManagerStructPropertyNode::GenerateUnsignedEntryBox(FNumericProperty* Prop, int32 Offset)
{
	TOptional<U> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
	U SliderExponent, Delta;
	float ShiftMultiplier = 10.f;
	float CtrlMultiplier = 0.1f;
	bool SupportDynamicSliderMaxValue = false;
	bool SupportDynamicSliderMinValue = false;

	ExtractNumericMetadata(Prop, MinValue, MaxValue, SliderMinValue, SliderMaxValue, SliderExponent, Delta, ShiftMultiplier, CtrlMultiplier, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

	return SNew(SNumericEntryBox<uint64>)
	.AllowSpin(true)
	.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
	.OnBeginSliderMovement(this, &FVariantManagerStructPropertyNode::OnBeginSliderMovement, Prop)
	.OnEndSliderMovement(this, &FVariantManagerStructPropertyNode::OnUnsignedEndSliderMovement, Prop, Offset)
	.OnValueChanged(this, &FVariantManagerStructPropertyNode::OnUnsignedValueChanged, Prop)
	.OnValueCommitted(this, &FVariantManagerStructPropertyNode::OnUnsignedPropCommitted, Prop, Offset)
	.Value(this, &FVariantManagerStructPropertyNode::GetUnsignedValueFromCache, Prop)
	.UndeterminedString(LOCTEXT("MultipleValuesLabel", "Multiple Values"))
	.ShiftMultiplier(ShiftMultiplier)
	.CtrlMultiplier(CtrlMultiplier)
	.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
	.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
	.MinValue(GET_OR_EMPTY(MinValue, uint64))
	.MaxValue(GET_OR_EMPTY(MaxValue, uint64))
	.MinSliderValue(GET_OR_EMPTY(SliderMinValue, uint64))
	.MaxSliderValue(GET_OR_EMPTY(SliderMaxValue, uint64))
	.SliderExponent(SliderExponent)
	.Delta(Delta)
	.LabelPadding( FMargin( 3 ) )
	.LabelLocation( SNumericEntryBox<uint64>::ELabelLocation::Inside )
	.Label()
	[
		GetLabel(Prop)
	];
}

FVariantManagerStructPropertyNode::FVariantManagerStructPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager)
	: FVariantManagerPropertyNode(InPropertyValues, InVariantManager)
{
}

TSharedPtr<SWidget> FVariantManagerStructPropertyNode::GetPropertyValueWidget()
{
	if (PropertyValues.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "PropertyNode has no UPropertyValues!");
		return SNew(SBox);
	}

	// Check to see if we have all valid, equal UPropertyValues
	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();
	uint32 FirstPropHash = FirstPropertyValue->GetPropertyPathHash();
	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (!PropertyValue.IsValid())
		{
			UE_LOGF(LogVariantManager, Error, "PropertyValue was invalid!");
			return SNew(SBox);
		}

		if (PropertyValue.Get()->GetPropertyPathHash() != FirstPropHash)
		{
			UE_LOGF(LogVariantManager, Error, "A PropertyNode's PropertyValue array describes properties with different paths!");
			return SNew(SBox);
		}
	}

	// If all properties fail to resolve, just give back a "Failed to resolve" text block
	bool bAtLeastOneResolved = false;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		UPropertyValue* PropValRaw = WeakPropertyValue.Get();
		if (PropValRaw->Resolve())
		{
			if (!PropValRaw->HasRecordedData())
			{
				PropValRaw->RecordDataFromResolvedObject();
			}

			bAtLeastOneResolved = true;
		}
	}
	if(!bAtLeastOneResolved)
	{
		return GetFailedToResolveWidget(FirstPropertyValue);
	}

	TArray<uint8> FirstRecordedData = FirstPropertyValue->GetRecordedData();

	UScriptStruct* StructClassToEdit = FirstPropertyValue->GetStructPropertyStruct();

	// Convert the rotator to a vector so that it shows as XYZ instead of Row, Pitch and Yaw
	bool bIsRotator = (StructClassToEdit->GetFName() == NAME_Rotator);
	if (bIsRotator)
	{
		static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
		StructClassToEdit = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector"));
	}

	TSharedPtr<SHorizontalBox> HorizBox = SNew(SHorizontalBox);
	FMargin CommonMargin = FMargin(0.0f, 0.0f, 3.0f, 0.0f);

	FMargin LastMargin = CommonMargin;
	LastMargin.Right = 4.0f;

	FNumericProperty* LastProp = nullptr;
	for (TFieldIterator<FNumericProperty> NumPropIter(StructClassToEdit); NumPropIter; ++NumPropIter)
	{
		LastProp = *NumPropIter;
	}

	for (TFieldIterator<FNumericProperty> NumPropIter(StructClassToEdit); NumPropIter; ++NumPropIter)
	{
		FNumericProperty* Prop = *NumPropIter;

		// Check if we have the same value on all UPropertyValue, for each separate child property
		int32 Size = Prop->GetElementSize();
		int32 Offset = Prop->GetOffset_ForInternal();
		bool bSameValue = true;

		// For rotators, this widget will display an actual vector so we need to switch up
		// the size and offsets
		const size_t RotatorElementSize = sizeof(FRotator().Roll);
		if (bIsRotator)
		{
			const size_t VectorElementSize = sizeof( FVector().X );

			switch (Offset)
			{
			case 0: // Edit to X --> modify roll
				Offset = 2 * RotatorElementSize;
				break;
			case VectorElementSize: // Edit to Y --> modify pitch
				Offset = 0;
				break;
			case VectorElementSize * 2: // Edit to Z --> modify yaw
				Offset = RotatorElementSize;
				break;
			default:
				check(false);
				break;
			}
		}

		// This is not trivial because if we're a rotator property the standard is to actually show a vector
		// widget, but the vector widget expects to read/write from doubles, while rotators still hold floats
		const size_t RecordedElementSize = bIsRotator ? RotatorElementSize : Prop->GetElementSize();

		FMargin& MarginToUse = (Prop == LastProp) ? LastMargin : CommonMargin;

		if (Prop->IsFloatingPoint())
		{
			TOptional<double>& StoredValue = FloatValues.FindOrAdd(Prop);
			StoredValue = GetFloatValueFromPropertyValue(Prop, RecordedElementSize, Offset);

			TSharedPtr<SWidget> EntryBox = nullptr;
			switch (Size)
			{
			case sizeof(float):
				EntryBox = GenerateFloatEntryBox<float>(Prop, RecordedElementSize, Offset);
				break;
			case sizeof(double):
				EntryBox = GenerateFloatEntryBox<double>(Prop, RecordedElementSize, Offset);
				break;
			default:
				break;
			}

			if (EntryBox.IsValid())
			{
				HorizBox->AddSlot()
					.Padding(MarginToUse)
					[
						EntryBox.ToSharedRef()
					];
			}
		}
		// Signed int
		else if (Prop->CanHoldValue(-1))
		{
			TOptional<int64>& StoredValue = SignedValues.FindOrAdd(Prop);
			StoredValue = GetSignedValueFromPropertyValue(Prop, Offset);

			TSharedPtr<SWidget> EntryBox = nullptr;
			switch (Size)
			{
			case sizeof(int8):
				EntryBox = GenerateSignedEntryBox<int8>(Prop, Offset);
				break;
			case sizeof(int16):
				EntryBox = GenerateSignedEntryBox<int16>(Prop, Offset);
				break;
			case sizeof(int32):
				EntryBox = GenerateSignedEntryBox<int32>(Prop, Offset);
				break;
			case sizeof(int64):
				EntryBox = GenerateSignedEntryBox<int64>(Prop, Offset);
				break;
			default:
				break;
			}

			if (EntryBox.IsValid())
			{
				HorizBox->AddSlot()
					.Padding(MarginToUse)
					[
						EntryBox.ToSharedRef()
					];
			}
		}
		// Unsigned int
		else
		{
			TOptional<uint64>& StoredValue = UnsignedValues.FindOrAdd(Prop);
			StoredValue = GetUnsignedValueFromPropertyValue(Prop, Offset);

			TSharedPtr<SWidget> EntryBox = nullptr;
			switch (Size)
			{
			case sizeof(uint8):
				EntryBox = GenerateUnsignedEntryBox<uint8>(Prop, Offset);
				break;
			case sizeof(uint16):
				EntryBox = GenerateUnsignedEntryBox<uint16>(Prop, Offset);
				break;
			case sizeof(uint32):
				EntryBox = GenerateUnsignedEntryBox<uint32>(Prop, Offset);
				break;
			case sizeof(uint64):
				EntryBox = GenerateUnsignedEntryBox<uint64>(Prop, Offset);
				break;
			default:
				break;
			}

			if (EntryBox.IsValid())
			{
				HorizBox->AddSlot()
					.Padding(MarginToUse)
					[
						EntryBox.ToSharedRef()
					];
			}
		}
	}

	return	SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f)) // Extra padding to match the internal padding of single property nodes
			[
				HorizBox.ToSharedRef()
			];
}

void FVariantManagerStructPropertyNode::OnFloatPropCommitted(double InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset)
{
	if (!Prop || PropertyValues.Num() < 1 || !PropertyValues[0].IsValid())
	{
		return;
	}

	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();

	const TArray<uint8>& RecordedBytes = FirstPropertyValue->GetRecordedData();

	TArray<uint8> InValueBytes;
	InValueBytes.SetNumUninitialized( RecordedElementSize );

	switch ( RecordedElementSize )
	{
	case sizeof(float):
	{
		const float OldValue = *( reinterpret_cast< const float* >( RecordedBytes.GetData() + Offset ) );
		const float CastValue = static_cast< float >( InValue );

		if ( OldValue == CastValue )
		{
			return;
		}

		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), RecordedElementSize );
		break;
	}
	case sizeof(double):
	{
		const double OldValue = *( reinterpret_cast< const double* >( RecordedBytes.GetData() + Offset ) );

		if ( OldValue == InValue )
		{
			return;
		}

		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&InValue), RecordedElementSize );
		break;
	}
	default:
		UE_LOGF(LogVariantManager, Error, "Float property has unhandled size %d!", RecordedElementSize );
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("EditCapturedProperty", "Edit captured property '{0}'"),
		FText::FromName(FirstPropertyValue->GetPropertyName())));

	TOptional<double>& StoredVal = FloatValues.FindOrAdd(Prop);
	StoredVal = InValue;

	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			PropertyValue.Get()->SetRecordedData(InValueBytes.GetData(), RecordedElementSize, Offset);
		}
	}

	RecordButton->SetVisibility(GetRecordButtonVisibility());
	ResetButton->SetVisibility(GetResetButtonVisibility());
}

void FVariantManagerStructPropertyNode::OnSignedPropCommitted(int64 InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 Offset)
{
	if (!Prop || PropertyValues.Num() < 1 || !PropertyValues[0].IsValid())
	{
		return;
	}

	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();
	int32 Size = Prop->GetElementSize();
	TArray<uint8> InValueBytes;
	InValueBytes.SetNumUninitialized(Size);

	// Don't create a transaction or re-copy the data if its the same
	const TArray<uint8>& RecordedBytes = FirstPropertyValue->GetRecordedData();
	int64 OldValue = Prop->GetSignedIntPropertyValue(RecordedBytes.GetData() + Offset);
	if (OldValue == InValue)
	{
		return;
	}

	switch (Size)
	{
	case sizeof(int8):
	{
		int8 CastValue = static_cast<int8>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
	}
	case sizeof(int16):
	{
		int16 CastValue = static_cast<int16>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	case sizeof(int32):
	{
		int32 CastValue = static_cast<int32>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	case sizeof(int64):
	{
		int64 CastValue = static_cast<int64>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	default:
		UE_LOGF(LogVariantManager, Error, "Signed int property has unhandled size %d!", Size);
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("StructPropertyNodeUpdateRecordedData", "Capture new data from property capture '{0}'"),
		FText::FromName(FirstPropertyValue->GetPropertyName())));

	TOptional<int64>& StoredVal = SignedValues.FindOrAdd(Prop);
	StoredVal = InValue;

	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			PropertyValue.Get()->SetRecordedData(InValueBytes.GetData(), Size, Offset);
		}
	}

	RecordButton->SetVisibility(GetRecordButtonVisibility());
	ResetButton->SetVisibility(GetResetButtonVisibility());
}

void FVariantManagerStructPropertyNode::OnUnsignedPropCommitted(uint64 InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 Offset)
{
	if (!Prop || PropertyValues.Num() < 1 || !PropertyValues[0].IsValid())
	{
		return;
	}

	UPropertyValue* FirstPropertyValue = PropertyValues[0].Get();
	int32 Size = Prop->GetElementSize();
	TArray<uint8> InValueBytes;
	InValueBytes.SetNumUninitialized(Size);

	// Don't create a transaction or re-copy the data if its the same
	const TArray<uint8>& RecordedBytes = FirstPropertyValue->GetRecordedData();
	uint64 OldValue = Prop->GetUnsignedIntPropertyValue(RecordedBytes.GetData() + Offset);
	if (OldValue == InValue)
	{
		return;
	}

	switch (Size)
	{
	case sizeof(uint8):
	{
		uint8 CastValue = static_cast<uint8>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
	}
	case sizeof(uint16):
	{
		uint16 CastValue = static_cast<uint16>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	case sizeof(uint32):
	{
		uint32 CastValue = static_cast<uint32>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	case sizeof(uint64):
	{
		uint64 CastValue = static_cast<uint64>(InValue);
		FMemory::Memcpy(InValueBytes.GetData(), (uint8*)(&CastValue), Size);
		break;
	}
	default:
		UE_LOGF(LogVariantManager, Error, "Unsigned int property has unhandled size %d!", Size);
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("StructPropertyNodeUpdateRecordedData", "Capture new data from property capture '{0}'"),
		FText::FromName(FirstPropertyValue->GetPropertyName())));

	TOptional<uint64>& StoredVal = UnsignedValues.FindOrAdd(Prop);
	StoredVal = InValue;

	for (TWeakObjectPtr<UPropertyValue> PropertyValue : PropertyValues)
	{
		if (PropertyValue.IsValid())
		{
			PropertyValue.Get()->SetRecordedData(InValueBytes.GetData(), Size, Offset);
		}
	}

	RecordButton->SetVisibility(GetRecordButtonVisibility());
	ResetButton->SetVisibility(GetResetButtonVisibility());
}

TOptional<double> FVariantManagerStructPropertyNode::GetFloatValueFromPropertyValue(FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset) const
{
	int32 NumPropVals = PropertyValues.Num();
	if (Prop == nullptr || NumPropVals < 1)
	{
		return TOptional<double>();
	}

	const TArray<uint8>* FirstRecordedData = nullptr;

	bool bSameValue = true;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		// If one of them is invalid, we'll just show the "Multiple values" string anyway
		if (!WeakPropertyValue.IsValid())
		{
			FirstRecordedData = nullptr;
			break;
		}

		const TArray<uint8>& RecordedData = WeakPropertyValue.Get()->GetRecordedData();
		if (FirstRecordedData == nullptr)
		{
			FirstRecordedData = &RecordedData;
		}
		else
		{
			for (int32 Index = 0; Index < RecordedElementSize; Index++)
			{
				if ((*FirstRecordedData)[Offset + Index] != RecordedData[Offset + Index])
				{
					bSameValue = false;
					break;
				}
			}
		}

		if (!bSameValue)
		{
			FirstRecordedData = nullptr;
			break;
		}
	}

	if ( FirstRecordedData )
	{
		if ( RecordedElementSize == sizeof( float ) )
		{
			return *( reinterpret_cast< const float* >( FirstRecordedData->GetData() + Offset ) );
		}
		else if ( RecordedElementSize == sizeof( double ) )
		{
			return *( reinterpret_cast< const double* >( FirstRecordedData->GetData() + Offset ) );
		}
	}

	return TOptional<double>();
}

TOptional<int64> FVariantManagerStructPropertyNode::GetSignedValueFromPropertyValue(FNumericProperty* Prop, int32 Offset) const
{
	int32 NumPropVals = PropertyValues.Num();
	if (Prop == nullptr || NumPropVals < 1)
	{
		return TOptional<int64>();
	}

	int32 Size = Prop->GetElementSize();

	const TArray<uint8>* FirstRecordedData = nullptr;

	bool bSameValue = true;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		// If one of them is invalid, we'll just show the "Multiple values" string anyway
		if (!WeakPropertyValue.IsValid())
		{
			FirstRecordedData = nullptr;
			break;
		}

		const TArray<uint8>& RecordedData = WeakPropertyValue.Get()->GetRecordedData();
		if (FirstRecordedData == nullptr)
		{
			FirstRecordedData = &RecordedData;
		}
		else
		{
			for (int32 Index = 0; Index < Size; Index++)
			{
				if ((*FirstRecordedData)[Offset + Index] != RecordedData[Offset + Index])
				{
					bSameValue = false;
					break;
				}
			}
		}

		if (!bSameValue)
		{
			FirstRecordedData = nullptr;
			break;
		}
	}

	if (FirstRecordedData == nullptr)
	{
		return TOptional<int64>();
	}
	else
	{
		return Prop->GetSignedIntPropertyValue(FirstRecordedData->GetData() + Offset);
	}
}

TOptional<uint64> FVariantManagerStructPropertyNode::GetUnsignedValueFromPropertyValue(FNumericProperty* Prop, int32 Offset) const
{
	int32 NumPropVals = PropertyValues.Num();
	if (Prop == nullptr || NumPropVals < 1)
	{
		return TOptional<uint64>();
	}

	int32 Size = Prop->GetElementSize();

	const TArray<uint8>* FirstRecordedData = nullptr;

	bool bSameValue = true;
	for (TWeakObjectPtr<UPropertyValue> WeakPropertyValue : PropertyValues)
	{
		// If one of them is invalid, we'll just show the "Multiple values" string anyway
		if (!WeakPropertyValue.IsValid())
		{
			FirstRecordedData = nullptr;
			break;
		}

		const TArray<uint8>& RecordedData = WeakPropertyValue.Get()->GetRecordedData();
		if (FirstRecordedData == nullptr)
		{
			FirstRecordedData = &RecordedData;
		}
		else
		{
			for (int32 Index = 0; Index < Size; Index++)
			{
				if ((*FirstRecordedData)[Offset + Index] != RecordedData[Offset + Index])
				{
					bSameValue = false;
					break;
				}
			}
		}

		if (!bSameValue)
		{
			FirstRecordedData = nullptr;
			break;
		}
	}

	if (FirstRecordedData == nullptr)
	{
		return TOptional<uint64>();
	}
	else
	{
		return Prop->GetUnsignedIntPropertyValue(FirstRecordedData->GetData() + Offset);
	}
}

TOptional<double> FVariantManagerStructPropertyNode::GetFloatValueFromCache(FNumericProperty* Prop) const
{
	if (const TOptional<double>* FoundValue = FloatValues.Find(Prop))
	{
		return *FoundValue;
	}

	return TOptional<double>();
}

TOptional<int64> FVariantManagerStructPropertyNode::GetSignedValueFromCache(FNumericProperty* Prop) const
{
	if (const TOptional<int64>* FoundValue = SignedValues.Find(Prop))
	{
		return *FoundValue;
	}

	return TOptional<int64>();
}

TOptional<uint64> FVariantManagerStructPropertyNode::GetUnsignedValueFromCache(FNumericProperty* Prop) const
{
	if (const TOptional<uint64>* FoundValue = UnsignedValues.Find(Prop))
	{
		return *FoundValue;
	}

	return TOptional<uint64>();
}

void FVariantManagerStructPropertyNode::OnBeginSliderMovement(FNumericProperty* Prop)
{
	bIsUsingSlider = true;
}

void FVariantManagerStructPropertyNode::OnFloatEndSliderMovement(double LastValue, FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset)
{
	bIsUsingSlider = false;
	OnFloatPropCommitted(LastValue, ETextCommit::Type::Default, Prop, RecordedElementSize, Offset);
}

void FVariantManagerStructPropertyNode::OnSignedEndSliderMovement(int64 LastValue, FNumericProperty* Prop, int32 Offset)
{
	bIsUsingSlider = false;
	OnSignedPropCommitted(LastValue, ETextCommit::Type::Default, Prop, Offset);
}

void FVariantManagerStructPropertyNode::OnUnsignedEndSliderMovement(uint64 LastValue, FNumericProperty* Prop, int32 Offset)
{
	bIsUsingSlider = false;
	OnUnsignedPropCommitted(LastValue, ETextCommit::Type::Default, Prop, Offset);
}

void FVariantManagerStructPropertyNode::OnFloatValueChanged(double NewValue, FNumericProperty* Prop)
{
	if (!bIsUsingSlider)
	{
		return;
	}

	TOptional<double>& StoredVal = FloatValues.FindOrAdd(Prop);
	StoredVal = NewValue;
}

void FVariantManagerStructPropertyNode::OnSignedValueChanged(int64 NewValue, FNumericProperty* Prop)
{
	if (!bIsUsingSlider)
	{
		return;
	}

	TOptional<int64>& StoredVal = SignedValues.FindOrAdd(Prop);
	StoredVal = NewValue;
}

void FVariantManagerStructPropertyNode::OnUnsignedValueChanged(uint64 NewValue, FNumericProperty* Prop)
{
	if (!bIsUsingSlider)
	{
		return;
	}

	TOptional<uint64>& StoredVal = UnsignedValues.FindOrAdd(Prop);
	StoredVal = NewValue;
}

#undef LOCTEXT_NAMESPACE
