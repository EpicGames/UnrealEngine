// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerPropertyNode.h"

class SVariantManagerTableRow;
class UPropertyValue;
class FNumericProperty;

class FVariantManagerStructPropertyNode
	: public FVariantManagerPropertyNode
{
public:

	FVariantManagerStructPropertyNode(TArray<UPropertyValue*> InPropertyValues, TWeakPtr<FVariantManager> InVariantManager);

protected:

	virtual TSharedPtr<SWidget> GetPropertyValueWidget() override;

private:

	void OnFloatPropCommitted(double InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset);
	void OnSignedPropCommitted(int64 InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 Offset);
	void OnUnsignedPropCommitted(uint64 InValue, ETextCommit::Type InCommitType, FNumericProperty* Prop, int32 Offset);

	TOptional<double> GetFloatValueFromPropertyValue(FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset) const;
	TOptional<int64> GetSignedValueFromPropertyValue(FNumericProperty* Prop, int32 Offset) const;
	TOptional<uint64> GetUnsignedValueFromPropertyValue(FNumericProperty* Prop, int32 Offset) const;

	TOptional<double> GetFloatValueFromCache(FNumericProperty* Prop) const;
	TOptional<int64> GetSignedValueFromCache(FNumericProperty* Prop) const;
	TOptional<uint64> GetUnsignedValueFromCache(FNumericProperty* Prop) const;

	void OnBeginSliderMovement(FNumericProperty* Prop);

	void OnFloatEndSliderMovement(double LastValue, FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset);
	void OnSignedEndSliderMovement(int64 LastValue, FNumericProperty* Prop, int32 Offset);
	void OnUnsignedEndSliderMovement(uint64 LastValue, FNumericProperty* Prop, int32 Offset);

	void OnFloatValueChanged(double NewValue, FNumericProperty* Prop);
	void OnSignedValueChanged(int64 NewValue, FNumericProperty* Prop);
	void OnUnsignedValueChanged(uint64 NewValue, FNumericProperty* Prop);

	template <typename F>
	TSharedRef<SWidget> GenerateFloatEntryBox(FNumericProperty* Prop, int32 RecordedElementSize, int32 Offset);
	template <typename S>
	TSharedRef<SWidget> GenerateSignedEntryBox(FNumericProperty* Prop, int32 Offset);
	template <typename U>
	TSharedRef<SWidget> GenerateUnsignedEntryBox(FNumericProperty* Prop, int32 Offset);

	bool bIsUsingSlider = false;

	// Cached values to be used for GetXValue and OnXValueChanged
	TMap<FNumericProperty*, TOptional<double>> FloatValues;
	TMap<FNumericProperty*, TOptional<int64>> SignedValues;
	TMap<FNumericProperty*, TOptional<uint64>> UnsignedValues;
};
