// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DisplayNodes/VariantManagerActorNode.h"
#include "Engine/Texture2D.h"
#include "SVariantManager.h"
#include "ThumbnailGenerator.h"
#include "Variant.h"
#include "VariantManager.h"
#include "VariantManagerDragDropOp.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerSelection.h"
#include "VariantManagerStyle.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"

#include "AssetThumbnail.h"
#include "ConsoleVariables.h"
#include "LevelVariantSets.h"
#include "SAssetDropTarget.h"
#include "Brushes/SlateImageBrush.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "VariantManagerVariantNode"

FVariantManagerVariantNode::FVariantManagerVariantNode( UVariant& InVariant, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree  )
	: FVariantManagerDisplayNode(InParentNode, InParentTree )
	, Variant(InVariant)
{

}

FReply FVariantManagerVariantNode::OnDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (VarMan.IsValid())
	{
		TSharedPtr<SVariantManager> SVarMan = VarMan->GetVariantManagerWidget();
		if (SVarMan.IsValid())
		{
			SVarMan->SwitchOnVariant(&Variant);
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> FVariantManagerVariantNode::GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow)
{
	const bool bUseNewAssetPicker = UE::VariantManager::Get_CVar_VariantManager_AssetPicker().GetValueOnGameThread();

	FSlateFontInfo NodeFont = FAppStyle::GetFontStyle(bUseNewAssetPicker ? "NormalFont" : "Sequencer.AnimationOutliner.RegularFont");

	EditableLabel = SNew(SInlineEditableTextBlock)
		.IsReadOnly(this, &FVariantManagerDisplayNode::IsReadOnly)
		.Font(NodeFont)
		.OnTextCommitted(this, &FVariantManagerDisplayNode::HandleNodeLabelTextChanged)
		.Text(this, &FVariantManagerDisplayNode::GetDisplayName)
		.ToolTipText(this, &FVariantManagerDisplayNode::GetDisplayNameToolTipText)
		.Clipping(EWidgetClipping::ClipToBounds);

	RadioButton = SNew(SCheckBox)
		.HAlign(HAlign_Right)
		.Padding(FMargin(0))
		.Style(FVariantManagerStyle::Get(), "VariantManager.VariantRadioButton")
		.ToolTipText(LOCTEXT("ActivateVariantRadioToolTip", "Activate the variant"))
		.IsChecked(FVariantManagerVariantNode::IsRadioButtonChecked())
		.OnCheckStateChanged(this, &FVariantManagerVariantNode::OnRadioButtonStateChanged);

	return
	SNew(SBox)
	.HeightOverride(40)
	[
		SNew(SBorder)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.BorderImage(nullptr)
		.Padding(FMargin(26.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			[
				RadioButton.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SAssetDropTarget)
				.OnAreAssetsAcceptableForDrop(this, &FVariantManagerVariantNode::AreAssetsAcceptableForDrop)
				.OnAssetsDropped(this, &FVariantManagerVariantNode::HandleAssetsDropped)
				[
					SNew(SBox)
					.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
					[
						GetThumbnailWidget()
					]
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
			[
				EditableLabel.ToSharedRef()
			]
		]
	];
}

EVariantManagerNodeType FVariantManagerVariantNode::GetType() const
{
	return EVariantManagerNodeType::Variant;
}

bool FVariantManagerVariantNode::IsReadOnly() const
{
	return false;
}

FText FVariantManagerVariantNode::GetDisplayName() const
{
	return Variant.GetDisplayText();
}

void FVariantManagerVariantNode::SetDisplayName( const FText& NewDisplayName )
{
	FString NewDisplayNameStr = NewDisplayName.ToString();

	if ( Variant.GetDisplayText().ToString() != NewDisplayNameStr )
	{
		Variant.Modify();

		if (UVariantSet* Parent = Variant.GetParent())
		{
			NewDisplayNameStr = Parent->GetUniqueVariantName(NewDisplayNameStr);
			if (NewDisplayNameStr != NewDisplayName.ToString())
			{
				FNotificationInfo Info(FText::Format(
					LOCTEXT("VariantNodeDuplicateNameNotification", "Variant set '{0}' already has a variant named '{1}'.\nNew name will be modified to '{2}' for uniqueness."),
					Parent->GetDisplayText(),
					NewDisplayName,
					FText::FromString(NewDisplayNameStr)));

				Info.ExpireDuration = 5.0f;
				Info.bUseLargeFont = false;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}

		Variant.SetDisplayText(FText::FromString(NewDisplayNameStr));
	}
}

void FVariantManagerVariantNode::HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT( "VariantManagerRenameVariantTransaction", "Rename variant to '{0}'" ), NewLabel));
	Variant.Modify();

	FVariantManagerDisplayNode::HandleNodeLabelTextChanged(NewLabel, CommitType);
}

bool FVariantManagerVariantNode::IsSelectable() const
{
	return true;
}

bool FVariantManagerVariantNode::CanDrag() const
{
	return true;
}

TOptional<EItemDropZone> FVariantManagerVariantNode::CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone ) const
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	// Dragging scene actors
	if (ActorDragDrop.IsValid())
	{
		UVariant* Var = &GetVariant();
		TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;

		VarMan->CanAddActorsToVariant(ActorDragDrop->Actors, Var, ActorsWeCanAdd);
		int32 NumActorsWeCanAdd = ActorsWeCanAdd.Num();

		// Get the FDecoratedDragDropOp so that we can use its non-virtual SetToolTip function and not FActorDragDropGraphEdOp's
		// We know this is valid because FActorDragDropGraphEdOp derives from it
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();

		if (NumActorsWeCanAdd > 0)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_BindActors", "Bind {0} {0}|plural(one=actor,other=actors) to variant '{1}'"),
				NumActorsWeCanAdd,
				GetVariant().GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return EItemDropZone::OntoItem;
		}
		else
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ActorsAlreadyBound", "Actors already bound to variant '{0}'!"),
				GetVariant().GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return TOptional<EItemDropZone>();
		}
	}
	// Dragging nodes from the variant manager itself
	else if (VarManDragDrop.IsValid())
	{
		UVariant* Var = &GetVariant();
		UVariantSet* ParentVarSet = Var->GetParent();

		TArray<TWeakObjectPtr<AActor>> DraggedBoundActors;
		TArray<UVariant*> DraggedVariants;
		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode);
				if (DraggedActorNode.IsValid())
				{
					UObject* DraggedObj = DraggedActorNode->GetObjectBinding()->GetObject();
					AActor* DraggedActor = Cast<AActor>(DraggedObj);

					if (DraggedActor)
					{
						DraggedBoundActors.AddUnique(DraggedActor);
					}
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> VarNode = StaticCastSharedRef<FVariantManagerVariantNode>(DraggedNode);
				if (VarNode.IsValid())
				{
					UVariant* VarNodeVariant = &VarNode->GetVariant();
					if (VarNodeVariant)
					{
						DraggedVariants.Add(VarNodeVariant);
					}
				}
			}
		}

		if (DraggedBoundActors.Num() > 0)
		{
			TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;

			VarMan->CanAddActorsToVariant(DraggedBoundActors, Var, ActorsWeCanAdd);
			int32 NumActorsWeCanAdd = ActorsWeCanAdd.Num();

			if (NumActorsWeCanAdd > 0)
			{
				FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

				FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ApplyActors", "{0} {1} actor {1}|plural(one=binding,other=bindings) to variant '{2}'"),
					ModifierKeysState.IsControlDown() ? LOCTEXT("Copy", "Copy") : LOCTEXT("Move", "Move"),
					NumActorsWeCanAdd,
					GetVariant().GetDisplayText());

				const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

				VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

				return EItemDropZone::OntoItem;
			}
			else
			{
				FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ActorsAlreadyBound", "Actors already bound to variant '{0}'!"),
					GetVariant().GetDisplayText());

				const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

				VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

				return TOptional<EItemDropZone>();
			}
		}
		else if (DraggedVariants.Num() > 0)
		{
			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ApplyVariants", "{0} {1} {1}|plural(one=variant,other=variants) to set '{2}'"),
				ModifierKeysState.IsControlDown() ? LOCTEXT("Copy", "Copy") : LOCTEXT("Move", "Move"),
				DraggedVariants.Num(),
				ParentVarSet->GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

			return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
		}
	}

	if (VarManDragDrop.IsValid())
	{
		VarManDragDrop->ResetToDefaultToolTip();
	}
	else if (ActorDragDrop.IsValid())
	{
		ActorDragDrop->ResetToDefaultToolTip();
	}
	return TOptional<EItemDropZone>();
}

void FVariantManagerVariantNode::Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone )
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return;
	}

	UVariant* Var = &GetVariant();

	if (ActorDragDrop.IsValid())
	{
		TArray<AActor*> Actors;
		Actors.Reserve(ActorDragDrop->Actors.Num());
		for (TWeakObjectPtr<AActor> Actor : ActorDragDrop->Actors)
		{
			AActor* RawActor = Actor.Get();
			if (RawActor->IsValidLowLevel())
			{
				Actors.Add(RawActor);
			}
		}

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("VariantNodeDropSceneActors", "Drop {0} scene {0}|plural(one=actor,other=actors) on variant '{1}'"),
			Actors.Num(),
			GetDisplayName()));

		VarMan->CreateObjectBindingsAndCaptures(Actors, {Var});
		VarMan->GetVariantManagerWidget()->RefreshActorList();
	}
	else if (VarManDragDrop.IsValid())
	{
		TArray<UVariantObjectBinding*> DraggedBindings;
		TSet<UObject*> DraggedActors; // Don't capture more than one binding to each actor, just keep first one we find
		TArray<UVariant*> DraggedVariants;

		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode);
				if (DraggedActorNode.IsValid())
				{
					UVariantObjectBinding* DraggedBinding = DraggedActorNode->GetObjectBinding().Get();
					if (DraggedBinding)
					{
						UObject* DraggedActor = DraggedBinding->GetObject();
						if (!DraggedActors.Contains(DraggedActor))
						{
							DraggedBindings.Add(DraggedBinding);
							DraggedActors.Add(DraggedActor);
						}
					}
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> VarNode = StaticCastSharedRef<FVariantManagerVariantNode>(DraggedNode);
				if (VarNode.IsValid())
				{
					UVariant* VarNodeVariant = &VarNode->GetVariant();
					if (VarNodeVariant)
					{
						DraggedVariants.Add(VarNodeVariant);
					}
				}
			}
		}

		if (DraggedBindings.Num() > 0)
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantNodeDropBindings", "Drop {0} actor {0}|plural(one=binding,other=bindings) on variant '{1}'"),
				DraggedBindings.Num(),
				GetDisplayName()));

			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
			if (ModifierKeysState.IsControlDown())
			{
				VarMan->DuplicateObjectBindings(DraggedBindings, Var);
			}
			else
			{
				VarMan->MoveObjectBindings(DraggedBindings, Var);
			}

			// Store selection to new bindings (nodes haven't been created yet so we must do this here)
			TSet<FString>& SelectedPaths = VarMan->GetSelection().GetSelectedNodePaths();
			for (UVariantObjectBinding* Binding : DraggedBindings)
			{
				SelectedPaths.Add(Binding->GetPathName());
			}

			VarMan->GetVariantManagerWidget()->RefreshActorList();
		}
		else if (DraggedVariants.Num() > 0)
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantNodeDropVariants", "Drop {0} {0}|plural(one=variant,other=variants) near variant '{1}'"),
				DraggedVariants.Num(),
				GetDisplayName()));

			UVariantSet* ParentVarSet = Var->GetParent();
			if (!ParentVarSet)
			{
				return;
			}

			int32 TargetIndex = ParentVarSet->GetVariantIndex(Var);
			if (TargetIndex != INDEX_NONE && ItemDropZone != EItemDropZone::AboveItem)
			{
				TargetIndex += 1;
			}

			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
			if(ModifierKeysState.IsControlDown())
			{
				VarMan->DuplicateVariants(DraggedVariants, Var->GetParent(), TargetIndex);
			}
			else
			{
				VarMan->MoveVariants(DraggedVariants, Var->GetParent(), TargetIndex);

				// Store selection to new bindings (nodes haven't been created yet so we must do this here)
				TSet<FString>& SelectedPaths = VarMan->GetSelection().GetSelectedNodePaths();
				for (UVariant* DraggedVariant : DraggedVariants)
				{
					SelectedPaths.Add(DraggedVariant->GetPathName());
				}
			}

			VarMan->GetVariantManagerWidget()->RefreshVariantTree();
		}
	}
}

void FVariantManagerVariantNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FVariantManagerDisplayNode::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection(TEXT("Variant"), LOCTEXT("VariantSectionText", "Variant"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().AddSelectedActorsCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().SwitchOnSelectedVariantCommand);
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection(TEXT("Thumbnail"), LOCTEXT("ThumbnailSectionText", "Thumbnail"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().CreateThumbnailCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().SelectThumbnailAssetCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().LoadThumbnailCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().ClearThumbnailCommand);
	MenuBuilder.EndSection();
}

UVariant& FVariantManagerVariantNode::GetVariant() const
{
	return Variant;
}

TSharedRef<SWidget> FVariantManagerVariantNode::GetThumbnailWidget()
{
	TSharedPtr<SWidget> ThumbnailWidget = nullptr;

	// Try using texture2d thumbnails
	if (UTexture2D* CreatedThumbnail = Variant.GetThumbnail())
	{
		if (!ImageBrush.IsValid() || ImageBrush->GetResourceObject() != CreatedThumbnail)
		{
			ImageBrush = MakeShareable(
				new FSlateImageBrush((UObject*)CreatedThumbnail, FVector2D(CreatedThumbnail->GetSizeX(), CreatedThumbnail->GetSizeY())));
		}

		if (ImageBrush.IsValid())
		{
			ThumbnailWidget = SNew(SImage).Image(ImageBrush.Get());
		}
	}
	else
	{
		ThumbnailWidget = SNew(SImage).Image(FVariantManagerStyle::Get().GetBrush("VariantManager.Variant"));
	}

	// Use default thumbnail graphic for Variants as a fallback
	if(!ThumbnailWidget.IsValid())
	{
		TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(&Variant, VARIANT_MANAGER_THUMBNAIL_SIZE, VARIANT_MANAGER_THUMBNAIL_SIZE, UThumbnailManager::Get().GetSharedThumbnailPool());
		ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
	}

	return SNew(SBox)
		.WidthOverride(32)
		.HeightOverride(32)
		[
			ThumbnailWidget.IsValid() ? ThumbnailWidget.ToSharedRef() : SNullWidget::NullWidget
		];
}

bool FVariantManagerVariantNode::AreAssetsAcceptableForDrop(TArrayView<FAssetData> AssetDatas) const
{
	if (AssetDatas.Num() == 1)
	{
		const FAssetData& AssetData = AssetDatas[0];
		if (UObject* Object = AssetData.GetAsset())
		{
			if (Cast<ULevelVariantSets>(Object))
			{	// We can't take LevelVariantSets as thumbnails
				return false;
			}

		    // Try to convert old thumbnails to a new thumbnail
		    FName ObjectName = *Object->GetFullName();
		    FThumbnailMap Map;
		    ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ObjectName}, Map);

		    if (FObjectThumbnail* OldThumbnail = Map.Find(ObjectName))
		    {
				return true;
			}
		}
	}

	return false;
}

void FVariantManagerVariantNode::HandleAssetsDropped(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas)
{
	if (AssetDatas.Num() == 1)
	{
		const FAssetData& AssetData = AssetDatas[0];
		if (UObject* Asset = AssetData.GetAsset())
		{
			FScopedTransaction Transaction(LOCTEXT("VariantNodeSetThumbnailTransaction", "Set thumbnail for variant"));
			Variant.Modify();
			Variant.SetThumbnailFromObject(Asset);
			if (TSharedPtr<FVariantManager> VariantManager = GetVariantManager().Pin())
			{
				if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManager->GetVariantManagerWidget())
				{
					VariantManagerWidget->RefreshVariantTree();
				}
			}
		}
	}
}
ECheckBoxState FVariantManagerVariantNode::IsRadioButtonChecked() const
{
	return Variant.IsActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVariantManagerVariantNode::OnRadioButtonStateChanged(ECheckBoxState NewState)
{
	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (VarMan.IsValid())
	{
		TSharedPtr<SVariantManager> SVarMan = VarMan->GetVariantManagerWidget();
		if (SVarMan.IsValid())
		{
			if (NewState == ECheckBoxState::Checked)
			{
				SVarMan->SwitchOnVariant(&Variant);
			}

			// TODO: Sometimes we don't need to do this...
			SVarMan->RefreshVariantTree();
		}
	}
}

#undef LOCTEXT_NAMESPACE
