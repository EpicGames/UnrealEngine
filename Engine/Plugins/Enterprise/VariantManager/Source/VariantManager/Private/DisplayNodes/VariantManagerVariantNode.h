// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerDisplayNode.h"


enum class ECheckBoxState : uint8;

class FDragDropEvent;
class FMenuBuilder;
class SCheckBox;
class SVariantManagerTableRow;
class UVariant;
enum class EItemDropZone;
struct FSlateImageBrush;

/** A variant manager display node representing a variant in the outliner. */
class FVariantManagerVariantNode : public FVariantManagerDisplayNode
{
public:
	FVariantManagerVariantNode( UVariant& InVariant, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree );

	// FVariantManagerDisplayNode interface
	virtual FReply OnDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual TSharedRef<SWidget> GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow) override;
	virtual EVariantManagerNodeType GetType() const override;
	virtual bool IsReadOnly() const override;
	virtual FText GetDisplayName() const override;
	virtual void SetDisplayName( const FText& NewDisplayName ) override;
	virtual void HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType) override;
	virtual bool IsSelectable() const override;
	virtual bool CanDrag() const override;
	virtual TOptional<EItemDropZone> CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const override;
	virtual void Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	//~ End FVariantManagerDisplayNode interface

	/** Gets the folder data for this display node. */
	UVariant& GetVariant() const;
	TSharedRef<SWidget> GetThumbnailWidget();

private:
	bool AreAssetsAcceptableForDrop(TArrayView<FAssetData> AssetDatas) const;
	void HandleAssetsDropped(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas);

private:

	ECheckBoxState IsRadioButtonChecked() const;
	void OnRadioButtonStateChanged(ECheckBoxState NewState);

	UVariant& Variant;

	TSharedPtr<SCheckBox> RadioButton;

	// We need to keep the brush alive while the thumbnail is rendered
	TSharedPtr<FSlateImageBrush> ImageBrush;
};
