// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayNodes/VariantManagerVariantSetNode.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DisplayNodes/VariantManagerActorNode.h"
#include "LevelVariantSets.h"
#include "SVariantManager.h"
#include "SVariantManagerTableRow.h"
#include "ThumbnailGenerator.h"
#include "Variant.h"
#include "VariantManager.h"
#include "VariantManagerDragDropOp.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerNodeTree.h"
#include "VariantManagerSelection.h"
#include "VariantManagerStyle.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"

#include "AssetThumbnail.h"
#include "SAssetDropTarget.h"
#include "Brushes/SlateImageBrush.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "VariantManagerVariantSetNode"

FVariantManagerVariantSetNode::FVariantManagerVariantSetNode( UVariantSet& InVariantSet, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree  )
	: FVariantManagerDisplayNode(InParentNode, InParentTree)
	, VariantSet(InVariantSet)
{
	ExpandedBackgroundBrush = FAppStyle::GetBrush("Sequencer.AnimationOutliner.TopLevelBorder_Expanded");
	CollapsedBackgroundBrush = FAppStyle::GetBrush("Sequencer.AnimationOutliner.TopLevelBorder_Collapsed");
	RowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row");
	RowStyle
		.SetEvenRowBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Header, 2.f, FStyleColors::Transparent, 1.f))
		.SetOddRowBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Header, 2.f, FStyleColors::Transparent, 1.f))
		.SetEvenRowBackgroundHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::SelectHover, 2.f))
		.SetOddRowBackgroundHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::SelectHover, 2.f));

	bExpanded = InVariantSet.IsExpanded();
}

const FTableRowStyle* FVariantManagerVariantSetNode::GetRowStyle() const
{
	return &RowStyle;
}

TSharedRef<SWidget> FVariantManagerVariantSetNode::GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow)
{
	FSlateFontInfo NodeFont = FAppStyle::GetFontStyle("BoldFont");

	EditableLabel = SNew(SInlineEditableTextBlock)
		.IsReadOnly(this, &FVariantManagerDisplayNode::IsReadOnly)
		.Font(NodeFont)
		.OnTextCommitted(this, &FVariantManagerDisplayNode::HandleNodeLabelTextChanged)
		.Text(this, &FVariantManagerDisplayNode::GetDisplayName)
		.ToolTipText(this, &FVariantManagerDisplayNode::GetDisplayNameToolTipText)
		.Clipping(EWidgetClipping::ClipToBounds);

	bool bIsFirstRowOfTree = false;
	TSharedPtr<FVariantManagerNodeTree> PinnedParentTree = ParentTree.Pin();
	if ( PinnedParentTree.IsValid() )
	{
		const TArray< TSharedRef<FVariantManagerDisplayNode> >& RootNodes = PinnedParentTree->GetRootNodes();
		if ( RootNodes.Num() > 0 && RootNodes[ 0 ] == SharedThis( this ) )
		{
			bIsFirstRowOfTree = true;
		}
	}

	return
	SNew( SBorder )
	.BorderImage(nullptr)
	[
		SNew(SBox)
		.HeightOverride(48)
		[
			SNew(SBorder)
			.BorderImage(nullptr)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				.MaxWidth(24.0f)
				.AutoWidth()
				[
					SNew(SExpanderArrow, InTableRow).IndentAmount( FVariantManagerStyle::Get().GetFloat( "VariantManager.Spacings.IndentAmount" ) )
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SAssetDropTarget)
					.OnAreAssetsAcceptableForDrop(this, &FVariantManagerVariantSetNode::AreAssetsAcceptableForDrop)
					.OnAssetsDropped(this, &FVariantManagerVariantSetNode::HandleAssetsDropped)
					[
						SNew(SBox)
						.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
						[
							GetThumbnailWidget()
						]
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
				.FillWidth(1.0f)
				[
					EditableLabel.ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.MaxWidth(24.0f)
				.AutoWidth()
				.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([this]
					{
						TSharedPtr<FVariantManager> VariantManagerPtr = GetVariantManager().Pin();
						if (VariantManagerPtr.IsValid())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("VariantSetNodeAddVariantTransaction", "Create a new variant for variant set '{0}'"),
								GetDisplayName()));

							VariantManagerPtr->CreateVariant(&GetVariantSet());
							if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManagerPtr->GetVariantManagerWidget())
							{
								VariantManagerWidget->RefreshVariantTree();
							}
						}

						return FReply::Handled();
					})
					.ContentPadding(FMargin(1, 0))
					.ToolTipText(LOCTEXT("AddVariantToolTip", "Add a new variant"))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
	];
}

TSharedRef<SWidget> FVariantManagerVariantSetNode::GetThumbnailWidget()
{
	TSharedPtr<SWidget> ThumbnailWidget = nullptr;

	// Try using texture2d thumbnails
	if (UTexture2D* CreatedThumbnail = VariantSet.GetThumbnail())
	{
		if (!ImageBrush.IsValid() || ImageBrush->GetResourceObject() != CreatedThumbnail)
		{
			ImageBrush = MakeShareable(
				new FSlateImageBrush((UObject*)CreatedThumbnail, FVector2D(CreatedThumbnail->GetSizeX(), CreatedThumbnail->GetSizeY())));
		}

		if (ImageBrush.IsValid())
		{
			ThumbnailWidget = SNew(SImage).Image(ImageBrush.Get());
		}
	}
	else
	{
		ThumbnailWidget = SNew(SImage).Image(FVariantManagerStyle::Get().GetBrush("VariantManager.VariantSet"));
	}

	// Use default thumbnail graphic for Variants as a fallback
	if (!ThumbnailWidget.IsValid())
	{
		ThumbnailPool = MakeShared<FAssetThumbnailPool>(1);

		TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(&VariantSet, VARIANT_MANAGER_THUMBNAIL_SIZE, VARIANT_MANAGER_THUMBNAIL_SIZE, ThumbnailPool);
		ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
	}

	return SNew(SBox)
		.WidthOverride(40)
		.HeightOverride(40)
		[
			ThumbnailWidget.IsValid() ? ThumbnailWidget.ToSharedRef() : SNullWidget::NullWidget
		];
}

void FVariantManagerVariantSetNode::SetExpansionState(bool bInExpanded)
{
	bExpanded = bInExpanded;

	VariantSet.SetExpanded(bInExpanded);
}

EVariantManagerNodeType FVariantManagerVariantSetNode::GetType() const
{
	return EVariantManagerNodeType::VariantSet;
}

bool FVariantManagerVariantSetNode::IsReadOnly() const
{
	return false;
}

FText FVariantManagerVariantSetNode::GetDisplayName() const
{
	return VariantSet.GetDisplayText();
}

void FVariantManagerVariantSetNode::SetDisplayName( const FText& NewDisplayName )
{
	FString NewDisplayNameStr = NewDisplayName.ToString();

	if ( VariantSet.GetDisplayText().ToString() != NewDisplayName.ToString() )
	{
		VariantSet.Modify();

		if (ULevelVariantSets* Parent = VariantSet.GetParent())
		{
			NewDisplayNameStr = Parent->GetUniqueVariantSetName(NewDisplayNameStr);
			if (NewDisplayNameStr != NewDisplayName.ToString())
			{
				FNotificationInfo Info(FText::Format(
					LOCTEXT("VariantNodeDuplicateNameNotification", "LevelVariantSets '{0}' already has a variant set named '{1}'.\nNew name will be modified to '{2}' for uniqueness."),
					FText::FromString(Parent->GetName()),
					NewDisplayName,
					FText::FromString(NewDisplayNameStr)));

				Info.ExpireDuration = 5.0f;
				Info.bUseLargeFont = false;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}

		VariantSet.SetDisplayText(FText::FromString(NewDisplayNameStr));
	}
}

void FVariantManagerVariantSetNode::HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT( "VariantManagerRenameVariantSetTransaction", "Rename variant set to '{0}'" ), NewLabel));
	VariantSet.Modify();

	FVariantManagerDisplayNode::HandleNodeLabelTextChanged(NewLabel, CommitType);
}

bool FVariantManagerVariantSetNode::IsSelectable() const
{
	return true;
}

bool FVariantManagerVariantSetNode::CanDrag() const
{
	return true;
}

TOptional<EItemDropZone> FVariantManagerVariantSetNode::CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	if (ActorDragDrop.IsValid())
	{
		// Convert to an array of raw actor pointers
		TArray<AActor*> Actors;
		Actors.Reserve(ActorDragDrop->Actors.Num());
		for (TWeakObjectPtr<AActor> Actor : ActorDragDrop->Actors)
		{
			AActor* RawActor = Actor.Get();
			if (RawActor)
			{
				Actors.Add(RawActor);
			}
		}

		int32 NumVarsThatCanAccept = 0;

		// Get all of our child variants that can receive these actors
		TArray<UVariant*> Vars;
		TSet<TWeakObjectPtr<AActor>> AllActorsWeCanAdd;
		for (const TSharedRef<FVariantManagerDisplayNode>& ChildNode : GetChildNodes())
		{
			if (ChildNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> ChildNodeAsVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(ChildNode);
				if (ChildNodeAsVarNode.IsValid())
				{
					UVariant* Var = &ChildNodeAsVarNode->GetVariant();

					TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;
					VarMan->CanAddActorsToVariant(ActorDragDrop->Actors, Var, ActorsWeCanAdd);

					if (ActorsWeCanAdd.Num() > 0)
					{
						NumVarsThatCanAccept += 1;

						AllActorsWeCanAdd.Append(ActorsWeCanAdd);
					}
				}
			}
		}

		int32 NumActorsWeCanAdd =  AllActorsWeCanAdd.Num();

		// Get the FDecoratedDragDropOp so that we can use its non-virtual SetToolTip function and not FActorDragDropGraphEdOp's
		// We know this is valid because FActorDragDropGraphEdOp derives from it
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();

		if (NumActorsWeCanAdd > 0 && NumVarsThatCanAccept > 0)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_BindActors", "Bind {0} {0}|plural(one=actor,other=actors) to {1} {1}|plural(one=variant,other=variants)"),
				NumActorsWeCanAdd,
				NumVarsThatCanAccept);

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return EItemDropZone::OntoItem;
		}
		else if (GetChildNodes().Num() < 1)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_NoVariantsInSet", "Variant set '{0}' has no variants!"),
				GetVariantSet().GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return TOptional<EItemDropZone>();
		}
		else
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ActorsAlreadyBound", "Actors already bound to all variants of set '{0}'!"),
				GetVariantSet().GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			return TOptional<EItemDropZone>();
		}
	}
	else if (VarManDragDrop.IsValid())
	{
		// Get all UVariantObjectBindings that are being dragged, but only one for each actor
		TArray<TWeakObjectPtr<AActor>> DraggedBoundActors;
		TArray<UVariant*> DraggedVariants;
		TArray<UVariantSet*> DraggedVariantSets;
		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode);
				if (DraggedActorNode.IsValid())
				{
					UObject* DraggedObj = DraggedActorNode->GetObjectBinding()->GetObject();
					AActor* DraggedActor = Cast<AActor>(DraggedObj);

					if (DraggedActor)
					{
						DraggedBoundActors.AddUnique(DraggedActor);
					}
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> DraggedVariantNode = StaticCastSharedRef<FVariantManagerVariantNode>(DraggedNode);
				if (DraggedVariantNode.IsValid())
				{
					DraggedVariants.Add(&DraggedVariantNode->GetVariant());
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::VariantSet)
			{
				TSharedPtr<FVariantManagerVariantSetNode> DraggedVariantSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(DraggedNode);
				if (DraggedVariantSetNode.IsValid())
				{
					DraggedVariantSets.Add(&DraggedVariantSetNode->GetVariantSet());
				}
			}
		}

		int32 NumVariants = GetChildNodes().Num();
		int32 NumDraggedBoundActors = DraggedBoundActors.Num();
		int32 NumDraggedVariants = DraggedVariants.Num();
		int32 NumDraggedVariantSets = DraggedVariantSets.Num();

		if (NumDraggedBoundActors > 0)
		{
			// See up to how many actors we can add to all our children
			TSet<TWeakObjectPtr<AActor>> AllActorsWeCanAdd;
			TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;

			for (const TSharedRef<FVariantManagerDisplayNode>& ChildNode : GetChildNodes())
			{
				if (ChildNode->GetType() == EVariantManagerNodeType::Variant)
				{
					TSharedPtr<FVariantManagerVariantNode> ChildVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(ChildNode);
					UVariant* Var = &ChildVarNode->GetVariant();

					VarMan->CanAddActorsToVariant(DraggedBoundActors, Var, ActorsWeCanAdd);
					AllActorsWeCanAdd.Append(ActorsWeCanAdd);
				}
			}

			int32 NumActorsWeCanAdd = AllActorsWeCanAdd.Num();

			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

			if (!ModifierKeysState.IsControlDown())
			{
				// Do nothing, can't move to multiple variants
				VarManDragDrop->ResetToDefaultToolTip();
			}
			else if (NumActorsWeCanAdd > 0)
			{
				FText NewHoverText = FText::Format( LOCTEXT("CanDrop_CopyActors", "Copy {0} actor {0}|plural(one=binding,other=bindings) to {1} {1}|plural(one=variant,other=variants)"),
					NumActorsWeCanAdd,
					NumVariants);

				const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

				VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

				return EItemDropZone::OntoItem;
			}
			else
			{
				FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ActorsAlreadyBound", "Actors already bound to all variants of set '{0}'!"),
					GetVariantSet().GetDisplayText());

				const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

				VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

				return TOptional<EItemDropZone>();
			}
		}
		else if (NumDraggedVariants > 0)
		{
			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ApplyVariantsToSet", "{0} {1} {1}|plural(one=variant,other=variants) to set '{2}'"),
				ModifierKeysState.IsControlDown() ? LOCTEXT("Copy", "Copy") : LOCTEXT("Move", "Move"),
				NumDraggedVariants,
				GetVariantSet().GetDisplayText());

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

			return EItemDropZone::OntoItem;
		}
		else if (NumDraggedVariantSets > 0)
		{
			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

			FText NewHoverText = FText::Format( LOCTEXT("CanDrop_ApplyVariantToSets", "{0} {1} variant {1}|plural(one=set,other=sets)"),
				ModifierKeysState.IsControlDown() ? LOCTEXT("Copy", "Copy") : LOCTEXT("Move", "Move"),
				NumDraggedVariants);

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			VarManDragDrop->SetToolTip(NewHoverText, NewHoverIcon);

			return ItemDropZone == EItemDropZone::AboveItem ? ItemDropZone : EItemDropZone::BelowItem;
		}
	}

	if (VarManDragDrop.IsValid())
	{
		VarManDragDrop->ResetToDefaultToolTip();
	}
	else if (ActorDragDrop.IsValid())
	{
		ActorDragDrop->ResetToDefaultToolTip();
	}
	return TOptional<EItemDropZone>();
}

void FVariantManagerVariantSetNode::Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone)
{
	TSharedPtr<FActorDragDropGraphEdOp> ActorDragDrop = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FVariantManagerDragDropOp> VarManDragDrop = DragDropEvent.GetOperationAs<FVariantManagerDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = GetVariantManager().Pin();
	if (!VarMan.IsValid())
	{
		return;
	}

	UVariantSet* VarSet = &GetVariantSet();

	if (ActorDragDrop.IsValid())
	{
		// Convert to an array of raw actor pointers
		TArray<AActor*> Actors;
		Actors.Reserve(ActorDragDrop->Actors.Num());
		for (TWeakObjectPtr<AActor> Actor : ActorDragDrop->Actors)
		{
			AActor* RawActor = Actor.Get();
			if (RawActor)
			{
				Actors.Add(RawActor);
			}
		}

		// Get all of our child variants that can receive these actors
		TArray<UVariant*> Vars;
		for (const TSharedRef<FVariantManagerDisplayNode>& ChildNode : GetChildNodes())
		{
			if (ChildNode->GetType() == EVariantManagerNodeType::Variant && ChildNode->CanDrop(DragDropEvent, ItemDropZone))
			{
				TSharedPtr<FVariantManagerVariantNode> ChildNodeAsVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(ChildNode);
				if (ChildNodeAsVarNode.IsValid())
				{
					Vars.Add(&ChildNodeAsVarNode->GetVariant());
				}
			}
		}

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("VariantSetNodeDropSceneActors", "Drop {0} scene {0}|plural(one=actor,other=actors) on variant set '{1}'"),
			Actors.Num(),
			GetDisplayName()));

		// Add bindings just once, or else it will spawn multiple popups and trigger multiple refreshes
		VarMan->CreateObjectBindingsAndCaptures(Actors, Vars);
		if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
		{ 
			VariantManagerWidget->RefreshActorList();
		}
	}
	else if (VarManDragDrop.IsValid())
	{
		// Get all UVariantObjectBindings that are being dragged, but only one for each actor
		TArray<UVariant*> DraggedVariants;
		TArray<UVariantSet*> DraggedVariantSets;
		TArray<UVariantObjectBinding*> DraggedBindings;
		TSet<UObject*> DraggedActors;
		for (const TSharedRef<FVariantManagerDisplayNode>& DraggedNode : VarManDragDrop->GetDraggedNodes())
		{
			if (DraggedNode->GetType() == EVariantManagerNodeType::Actor)
			{
				TSharedPtr<FVariantManagerActorNode> DraggedActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DraggedNode);
				if (DraggedActorNode.IsValid())
				{
					UVariantObjectBinding* DraggedBinding = DraggedActorNode->GetObjectBinding().Get();
					if (DraggedBinding)
					{
						UObject* DraggedActor = DraggedBinding->GetObject();
						if (!DraggedActors.Contains(DraggedActor))
						{
							DraggedBindings.Add(DraggedBinding);
							DraggedActors.Add(DraggedActor);
						}
					}
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> DraggedVariantNode = StaticCastSharedRef<FVariantManagerVariantNode>(DraggedNode);
				if (DraggedVariantNode.IsValid())
				{
					DraggedVariants.Add(&DraggedVariantNode->GetVariant());
				}
			}
			else if (DraggedNode->GetType() == EVariantManagerNodeType::VariantSet)
			{
				TSharedPtr<FVariantManagerVariantSetNode> DraggedVariantSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(DraggedNode);
				if (DraggedVariantSetNode.IsValid())
				{
					DraggedVariantSets.Add(&DraggedVariantSetNode->GetVariantSet());
				}
			}
		}

		int32 NumDraggedBindings = DraggedBindings.Num();
		int32 NumDraggedVariants = DraggedVariants.Num();
		int32 NumDraggedVariantSets = DraggedVariantSets.Num();

		if (NumDraggedBindings > 0)
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantSetNodeDropBindings", "Drop {0} actor {0}|plural(one=binding,other=bindings) on variant set '{1}'"),
				NumDraggedBindings,
				GetDisplayName()));

			// Copy our bindings to all our children
			for (const TSharedRef<FVariantManagerDisplayNode>& ChildNode : GetChildNodes())
			{
				if (ChildNode->GetType() == EVariantManagerNodeType::Variant)
				{
					TSharedPtr<FVariantManagerVariantNode> ChildVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(ChildNode);
					UVariant* Var = &ChildVarNode->GetVariant();

					VarMan->DuplicateObjectBindings(DraggedBindings, Var);
				}
				else if (ChildNode->GetType() == EVariantManagerNodeType::VariantSet)
				{
					TSharedPtr<FVariantManagerVariantSetNode> VariantSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(ChildNode);
					if (VariantSetNode.IsValid())
					{
						DraggedVariantSets.Add(&VariantSetNode->GetVariantSet());
					}
				}
			}

			// Store selection to new bindings (nodes haven't been created yet so we must do this here)
			TSet<FString>& SelectedPaths = VarMan->GetSelection().GetSelectedNodePaths();
			for (UVariantObjectBinding* Binding : DraggedBindings)
			{
				SelectedPaths.Add(Binding->GetPathName());
			}

			if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
			{
				VariantManagerWidget->RefreshActorList();
			}
		}
		else if (NumDraggedVariants > 0)
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantSetNodeDropVariants", "Drop {0} {0}|plural(one=variant,other=variants) on variant set '{1}'"),
				NumDraggedVariants,
				GetDisplayName()));

			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
			if (ModifierKeysState.IsControlDown())
			{
				VarMan->DuplicateVariants(DraggedVariants, &GetVariantSet());
			}
			else
			{
				VarMan->MoveVariants(DraggedVariants, &GetVariantSet());
			}

			// Store selection to new bindings (nodes haven't been created yet so we must do this here)
			TSet<FString>& SelectedPaths = VarMan->GetSelection().GetSelectedNodePaths();
			for (UVariant* Variant : DraggedVariants)
			{
				SelectedPaths.Add(Variant->GetPathName());
			}

			if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
			{
				VariantManagerWidget->RefreshVariantTree();
			}
		}
		else if (NumDraggedVariantSets > 0)
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantSetNodeDropVariantSets", "Drop {0} variant {0}|plural(one=set,other=sets) near variant set '{1}'"),
				NumDraggedVariantSets,
				GetDisplayName()));

			ULevelVariantSets* ParentLevelVarSet = VarSet->GetParent();
			if (!ParentLevelVarSet )
			{
				return;
			}

			int32 TargetIndex = ParentLevelVarSet ->GetVariantSetIndex(VarSet);
			if (TargetIndex != INDEX_NONE && ItemDropZone != EItemDropZone::AboveItem)
			{
				TargetIndex += 1;
			}

			FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
			if (ModifierKeysState.IsControlDown())
			{
				VarMan->DuplicateVariantSets(DraggedVariantSets, ParentLevelVarSet, TargetIndex);
			}
			else
			{
				VarMan->MoveVariantSets(DraggedVariantSets, ParentLevelVarSet, TargetIndex);

				// Store selection to new bindings (nodes haven't been created yet so we must do this here)
				TSet<FString>& SelectedPaths = VarMan->GetSelection().GetSelectedNodePaths();
				for (UVariantSet* DraggedVariantSet : DraggedVariantSets)
				{
					SelectedPaths.Add(DraggedVariantSet->GetPathName());
				}
			}

			if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
			{
				VariantManagerWidget->RefreshVariantTree();
			}
		}
	}
}

void FVariantManagerVariantSetNode::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FVariantManagerDisplayNode::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection("VariantSet", LOCTEXT("VariantSetSectionText", "VariantSet"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("VariantSetAddVariantContextText", "Add Variant"),
		LOCTEXT("VariantSetAddVariantContextToolTip", "Creates and adds a new variant to this variant set"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]
		{
			TSharedPtr<FVariantManager> VariantManagerPtr = GetVariantManager().Pin();
			if (VariantManagerPtr.IsValid())
			{
				FScopedTransaction Transaction(FText::Format(
					LOCTEXT("VariantSetNodeAddVariantTransaction", "Create a new variant for variant set '{0}'"),
					GetDisplayName()));

				VariantManagerPtr->CreateVariant(&GetVariantSet());
				if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManagerPtr->GetVariantManagerWidget())
				{
					VariantManagerWidget->RefreshVariantTree();
				}
			}
		})));
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("Thumbnail"), LOCTEXT("ThumbnailSectionText", "Thumbnail"));
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().CreateThumbnailCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().SelectThumbnailAssetCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().LoadThumbnailCommand);
	MenuBuilder.AddMenuEntry(FVariantManagerEditorCommands::Get().ClearThumbnailCommand);
	MenuBuilder.EndSection();
}

UVariantSet& FVariantManagerVariantSetNode::GetVariantSet() const
{
	return VariantSet;
}

bool FVariantManagerVariantSetNode::AreAssetsAcceptableForDrop(TArrayView<FAssetData> AssetDatas) const
{
	if (AssetDatas.Num() == 1)
	{
		const FAssetData& AssetData = AssetDatas[0];
		if (UObject* Object = AssetData.GetAsset())
		{
			if (Cast<ULevelVariantSets>(Object))
			{	// We can't take LevelVariantSets as thumbnails
				return false;
			}

		    // Try to convert old thumbnails to a new thumbnail
		    FName ObjectName = *Object->GetFullName();
		    FThumbnailMap Map;
		    ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ObjectName}, Map);

		    if (FObjectThumbnail* OldThumbnail = Map.Find(ObjectName))
		    {
				return true;
			}
		}
	}

	return false;
}

void FVariantManagerVariantSetNode::HandleAssetsDropped(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas)
{
	if (AssetDatas.Num() == 1)
	{
		const FAssetData& AssetData = AssetDatas[0];
		if (UObject* Asset = AssetData.GetAsset())
		{
			FScopedTransaction Transaction(LOCTEXT("VariantSetNodeSetThumbnailTransaction", "Set thumbnail for variant set"));
			VariantSet.Modify();
			VariantSet.SetThumbnailFromObject(Asset);
			if (TSharedPtr<FVariantManager> VariantManager = GetVariantManager().Pin())
			{
				if (TSharedPtr<SVariantManager> VariantManagerWidget = VariantManager->GetVariantManagerWidget())
				{
					VariantManagerWidget->RefreshVariantTree();
				}
			}
		}
	}
}

const FSlateBrush* FVariantManagerVariantSetNode::GetNodeBorderImage() const
{
	return IsExpanded() ? ExpandedBackgroundBrush : CollapsedBackgroundBrush;
}


#undef LOCTEXT_NAMESPACE
