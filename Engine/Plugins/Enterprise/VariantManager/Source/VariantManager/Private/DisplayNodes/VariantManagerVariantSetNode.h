// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerDisplayNode.h"


class FAssetThumbnailPool;
class FDragDropEvent;
class FMenuBuilder;
class SVariantManagerTableRow;
class UVariantSet;
enum class EItemDropZone;
struct FSlateImageBrush;

/** A variant manager display node representing a variant set in the outliner. */
class FVariantManagerVariantSetNode : public FVariantManagerDisplayNode
{
public:
	FVariantManagerVariantSetNode( UVariantSet& InVariantSet, TSharedPtr<FVariantManagerDisplayNode> InParentNode, TWeakPtr<FVariantManagerNodeTree> InParentTree );

	// FVariantManagerDisplayNode interface
	virtual const FTableRowStyle* GetRowStyle() const override;
	virtual EVariantManagerNodeType GetType() const override;
	virtual bool IsReadOnly() const override;
	virtual FText GetDisplayName() const override;
	virtual void SetDisplayName( const FText& NewDisplayName ) override;
	virtual void HandleNodeLabelTextChanged(const FText& NewLabel, ETextCommit::Type CommitType) override;
	virtual bool IsSelectable() const override;
	virtual bool CanDrag() const override;
	virtual TOptional<EItemDropZone> CanDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) const override;
	virtual void Drop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone) override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual const FSlateBrush* GetNodeBorderImage() const override;
	virtual TSharedRef<SWidget> GetCustomOutlinerContent(TSharedPtr<SVariantManagerTableRow> InTableRow) override;
	TSharedRef<SWidget> GetThumbnailWidget();
	virtual void SetExpansionState(bool bInExpanded) override;

	/** Gets the folder data for this display node. */
	UVariantSet& GetVariantSet() const;

private:
	bool AreAssetsAcceptableForDrop(TArrayView<FAssetData> AssetDatas) const;
	void HandleAssetsDropped(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas);

private:

	/** The movie scene folder data which this node represents. */
	UVariantSet& VariantSet;

	FTableRowStyle RowStyle;

	/** Different brushes so that the edges look good when expanded and collapsed */
	const FSlateBrush* ExpandedBackgroundBrush;
	const FSlateBrush* CollapsedBackgroundBrush;

	TSharedPtr<FAssetThumbnailPool> ThumbnailPool;

	// We need to keep the brush alive while the thumbnail is rendered
	TSharedPtr<FSlateBrush> ImageBrush;
};
