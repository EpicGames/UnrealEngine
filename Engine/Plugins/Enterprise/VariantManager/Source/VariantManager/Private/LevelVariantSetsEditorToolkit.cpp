// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelVariantSetsEditorToolkit.h"

#include "Widgets/Docking/SDockTab.h"
#include "LevelEditor.h"
#include "LevelVariantSets.h"
#include "SVariantManager.h"
#include "VariantManager.h"
#include "VariantManagerModule.h"
#include "SVariantManager.h"

#define LOCTEXT_NAMESPACE "LevelVariantSetsEditor"

FLevelVariantSetsEditorToolkit::FLevelVariantSetsEditorToolkit()
	: LevelVariantSets(nullptr)
{
}

FLevelVariantSetsEditorToolkit::~FLevelVariantSetsEditorToolkit()
{
	VariantManager->Close();

	if (CreatedTab.IsValid())
	{
		CreatedTab.Pin()->RequestCloseTab();
	}
}

void FLevelVariantSetsEditorToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULevelVariantSets* InLevelVariantSets)
{
	FName TabID = GetVariantManagerTabID();

	struct Local
	{
		static void OnVariantManagerClosed(TSharedRef<SDockTab> DockTab, TWeakPtr<IAssetEditorInstance> InLevelVariantSetsAssetEditor)
		{
			TSharedPtr<IAssetEditorInstance> AssetEditorInstance = InLevelVariantSetsAssetEditor.Pin();

			if (AssetEditorInstance.IsValid())
			{
				InLevelVariantSetsAssetEditor.Pin()->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
			}
		}
	};

	// create tab layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_LevelVariantSetsEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->Split
				(
					FTabManager::NewStack()
						->AddTab(TabID, ETabState::OpenedTab)
				)
		);

	LevelVariantSets = InLevelVariantSets;

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = false;

	VariantManager = FModuleManager::LoadModuleChecked<IVariantManagerModule>("VariantManager").CreateVariantManager(LevelVariantSets);

	// Create a new DockTab and add the variant manager widget to it
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> EditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	// Required, will cause the previous toolkit to close bringing down the variant manager and unsubscribing the
	// tab spawner. Without this, the InitAssetEditor call below will trigger an ensure as the VariantManager
	// tab ID will already be registered within EditorTabManager
	if (EditorTabManager->FindExistingLiveTab(TabID).IsValid())
	{
		EditorTabManager->TryInvokeTab(TabID)->RequestCloseTab();
	}

	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TEXT("VariantManagerApp"), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, LevelVariantSets);

	TSharedPtr<SDockTab> Tab = EditorTabManager->TryInvokeTab(TabID);
	if (Tab.IsValid())
	{
		Tab->SetContent(VariantManager->GetVariantManagerWidget().ToSharedRef());
		Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(&Local::OnVariantManagerClosed, TWeakPtr<IAssetEditorInstance>(SharedThis(this))));
	}
	CreatedTab = Tab;
}

FText FLevelVariantSetsEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Level Variant Sets");
}

FName FLevelVariantSetsEditorToolkit::GetToolkitFName() const
{
	return FName(TEXT("LevelVariantSetsEditor"));
}

FLinearColor FLevelVariantSetsEditorToolkit::GetWorldCentricTabColorScale() const
{
	return GetWorldCentricTabColorScaleStatic();
}

FString FLevelVariantSetsEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "VariantManager ").ToString();
}

FName FLevelVariantSetsEditorToolkit::GetVariantManagerTabID()
{
	return FName(TEXT("VariantManager_VariantManagerMain"));
}

FLinearColor FLevelVariantSetsEditorToolkit::GetWorldCentricTabColorScaleStatic()
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

TSharedRef<SDockTab> FLevelVariantSetsEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("VariantManagerMainTitle", "VariantManager"))
		.TabColorScale(GetWorldCentricTabColorScaleStatic())
		.TabRole(ETabRole::PanelTab);
}

bool FLevelVariantSetsEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return true;
}

bool FLevelVariantSetsEditorToolkit::CanFindInContentBrowser() const
{
	return false;
}

void FLevelVariantSetsEditorToolkit::FocusWindow(UObject* ObjectToFocusOn)
{
	FAssetEditorToolkit::FocusWindow(ObjectToFocusOn);

	if (CreatedTab.IsValid())
	{
		CreatedTab.Pin()->DrawAttention();
	}
}

#undef LOCTEXT_NAMESPACE
