// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;

class AActor;
class FMenuBuilder;
class FVariantManager;
enum class EMapChangeType : uint8;
class ULevelVariantSets;


class FLevelVariantSetsEditorToolkit
	: public FAssetEditorToolkit
	, public FGCObject
{
public:

	FLevelVariantSetsEditorToolkit();
	virtual ~FLevelVariantSetsEditorToolkit();

	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULevelVariantSets* LevelVariantSets);

	//~ FAssetEditorToolkit interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(LevelVariantSets);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FLevelVariantSetsEditorToolkit");
	}
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual bool CanFindInContentBrowser() const override;
	virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) override;

	//~ IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	// Static methods that can be used by the VariantManagerModule to spawn an empty tab when
	// doing Window > Variant Manager. Our regular methods also end up calling these
	static FName GetVariantManagerTabID();
	static FLinearColor GetWorldCentricTabColorScaleStatic();

private:
	/** Callback for spawning tabs. */
	static TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args);

	/** Level sequence for our edit operation. */
	TObjectPtr<ULevelVariantSets> LevelVariantSets;

	/** The VariantManager used by this editor. */
	TSharedPtr<FVariantManager> VariantManager;

	// Keep track of the tab we create so that we can close it when the asset closes
	TWeakPtr<SDockTab> CreatedTab;
};
