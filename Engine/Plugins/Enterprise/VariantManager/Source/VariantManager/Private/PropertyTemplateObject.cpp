// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyTemplateObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyTemplateObject)


UPropertyTemplateObject::UPropertyTemplateObject(const FObjectInitializer& Init)
{
}
