// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PropertyTemplateObject.generated.h"

/**
In order to use PropertyEditorModule.CreateSingleProperty we have to give it an object instance
and the name of the target property to edit. It will then iterate the object for a property with that
name and create a property editor widget.

This is very limiting when editing a single entry within an FArrayProperty, as the inner and the
array prop will have the same name, leading it to create an array editor. Also, since we have to
give it an instance, modifying the widget will automatically modify the object, which we may not
want, we may just want a property editor of a particular type.

This class is a hack around all that: It has an instance of most property types,
so that you can instantiate one of these and just pass along the name of the property type you want.
They are all be named Captured<propertyType> (e.g. CapturedFloatProperty, CapturedObjectProperty,
bCapturedBoolProperty) but you can use the helper function to get the name of the property you want.
*/

// TODO: Convert this into a static dictionary that maps to a small separate class for each property type
// Maybe even template it for array/map/set property types

UCLASS(Transient)
class UPropertyTemplateObject : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	static FName GetPropertyNameFromClass(const FFieldClass* PropertyType)
	{
		FString PropName;
		if (PropertyType == FBoolProperty::StaticClass())
		{
			PropName = TEXT("bCapturedBoolProperty");
		}
		else if(PropertyType == FClassProperty::StaticClass())
		{
			PropName = TEXT("CapturedObjectProperty");
		}
		else
		{
			PropName = TEXT("Captured") + PropertyType->GetName();
		}

		return FName(*PropName);
	}

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured byte property"))
	uint8 CapturedByteProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured uint16 property"))
	uint16 CapturedUInt16Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured uint32 property"))
	uint32 CapturedUInt32Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured uint16 property"))
	uint64 CapturedUInt64Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured int8 property"))
	int8 CapturedInt8Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured int16 property"))
	int16 CapturedInt16Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured int32 property"))
	int32 CapturedIntProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured int64 property"))
	int64 CapturedInt64Property;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured float property"))
	float CapturedFloatProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured double property"))
	double CapturedDoubleProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured boolean property"))
	bool bCapturedBoolProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured UObject property"))
	TObjectPtr<UObject> CapturedObjectProperty;

	//UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured Weak UObject property"))
	//TWeakObjectPtr<UObject> CapturedWeakObjectProperty;

	//UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured Lazy UObject property"))
	//TLazyObjectPtr<UObject> CapturedLazyObjectProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured Soft UObject property"))
	TSoftObjectPtr<UObject> CapturedSoftObjectProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured UInterface property"))
	FScriptInterface CapturedInterfaceProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured FName property"))
	FName CapturedNameProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured FString property"))
	FString CapturedStrProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured FText property"))
	FText CapturedTextProperty;

	UPROPERTY(Transient, EditAnywhere, Category="Template", meta=(ToolTip="Captured FVector property"))
	FVector CapturedVectorProperty;
};
