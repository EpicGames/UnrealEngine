// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDependencyRow.h"

#include "Framework/Views/TableViewMetadata.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SSeparator.h"

#include "LevelVariantSets.h"
#include "SVariantManager.h"
#include "Variant.h"
#include "VariantSet.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SDependencyRow"

const FName SDependencyRow::VisibilityColumn("Visibility");
const FName SDependencyRow::VariantSetColumn("VariantSet");
const FName SDependencyRow::VariantColumn("Variant");

void SDependencyRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FVariantDependencyModelPtr InDependencyModel, bool bInteractionEnabled)
{
	if (InDependencyModel.IsValid())
	{
		bIsDivider = InDependencyModel->bIsDivider;
		bIsDependent = InDependencyModel->bIsDependent;
		ParentVariantPtr = InDependencyModel->ParentVariant;
		Dependency = InDependencyModel->Dependency;
	}
	else
	{
		ParentVariantPtr.Reset();
		Dependency = nullptr;
	}

	RebuildVariantSetOptions();
	RebuildVariantOptions();

	SMultiColumnTableRow<FVariantDependencyModelPtr>::Construct(FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow")),
		InOwnerTableView);
}

TSharedRef<SWidget> SDependencyRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (bIsDivider)
	{
		if (InColumnName == VariantSetColumn || InColumnName == VariantColumn)
		{
			return SNew(SBox)
				.Padding(0, 15, 0, 15)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1)
					[	
						SNew(SSeparator)
						.Orientation(EOrientation::Orient_Horizontal)
						.Thickness(1)
						.SeparatorImage(FAppStyle::GetBrush("WhiteBrush"))
					]
				];
		}
	}
	else if (ParentVariantPtr.IsValid())
	{
		if (InColumnName == VisibilityColumn)
		{
			return SNew(SBox)
				.HeightOverride(21)
				.WidthOverride(21)
				[
					SNew(SButton)
					.IsFocusable(false)
					.ToolTipText(LOCTEXT("ToggleDependency", "Enable or disable this dependency"))
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ContentPadding(0.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Raw(this, &SDependencyRow::OnGetEyeIconVisibility)
					.OnClicked(this, &SDependencyRow::OnEnableRowToggled)
					[
						SNew(SImage)
						.Image_Lambda([this]()
						{
							if (Dependency && Dependency->bEnabled)
							{
								return FAppStyle::GetBrush("Level.VisibleIcon16x");
							}
							return FAppStyle::GetBrush("Level.NotVisibleIcon16x");
						})
					]
				];
		}
		else if (InColumnName == VariantSetColumn)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.HeightOverride(21)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.7f)
					[
						SNew(SComboBox<TSharedPtr<FText>>)
						.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("SimpleComboBox")))
						.OptionsSource(&VariantSetOptions)
						.OnGenerateWidget_Lambda([](TSharedPtr<FText> Item)
						{
							return SNew(STextBlock).Text(*Item);
						})
						.Content()
						[
							SNew(STextBlock)
							.Text(this, bIsDependent ? &SDependencyRow::GetDependentVariantSetText : &SDependencyRow::GetSelectedVariantSetOption)
						]
						.OnSelectionChanged(this, &SDependencyRow::OnSelectedVariantSetChanged)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(0.3f)
					[
						SNew(SSpacer)
					]
				];
		}
		else
		{ 
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.HeightOverride(21)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.7f)
					[
						SNew(SComboBox<TSharedPtr<FText>>)
						.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("SimpleComboBox")))
						.OptionsSource(&VariantOptions)
						.OnGenerateWidget_Lambda([](TSharedPtr<FText> Item)
						{
							return SNew(STextBlock).Text(*Item);
						})
						.Content()
						[
							SNew(STextBlock)
							.Text(this, bIsDependent ? &SDependencyRow::GetDependentVariantText  : &SDependencyRow::GetSelectedVariantOption)
						]
						.OnSelectionChanged(this, &SDependencyRow::OnSelectedVariantChanged)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(0.3f)
					[
						SNew(SSpacer)
					]
				];
		}
	}

	return SNew(SBox);
}

EVisibility SDependencyRow::OnGetEyeIconVisibility() const
{
	if (bIsDependent)
	{
		return EVisibility::Collapsed;
	}
	return (Dependency && !Dependency->bEnabled) || IsHovered() || IsSelected() ? EVisibility::Visible : EVisibility::Hidden;
}

void SDependencyRow::OnSelectedVariantSetChanged(TSharedPtr<FText> NewItem, ESelectInfo::Type SelectType)
{
	UVariant* ParentVariant = ParentVariantPtr.Get();
	if (!NewItem.IsValid() || !Dependency || !ParentVariant)
	{
		return;
	}

	ULevelVariantSets* LevelVariantSets = ParentVariant->GetTypedOuter<ULevelVariantSets>();
	if (!LevelVariantSets)
	{
		return;
	}

	for (UVariantSet* VariantSet : LevelVariantSets->GetVariantSets())
	{
		if (VariantSet && VariantSet->GetDisplayText().EqualTo(*NewItem))
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantSetDependencyChanged", "Make variant '{0}' depend on variant set '{0}'"),
				ParentVariant->GetDisplayText(),
				VariantSet->GetDisplayText()
			));

			ParentVariant->Modify();

			Dependency->VariantSet = VariantSet;
			Dependency->Variant = nullptr;

			// Automatically select a valid dependency variant if we have one.
			// The intent is to combine this with the fact that variant sets need to have at least one valid variant
			// to be pickable as a dependency in the first place.
			// These two facts together prevent us from getting to some invalid states when we could e.g. leave the
			// Variant part of the dependency as None, and then have another variant depend on this one.
			for ( UVariant* Variant : VariantSet->GetVariants() )
			{
				if ( ParentVariant->IsValidDependency( Variant ) )
				{
					Dependency->Variant = Variant;
					break;
				}
			}

			RebuildVariantOptions();
			return;
		}
	}
}

void SDependencyRow::OnSelectedVariantChanged(TSharedPtr<FText> NewItem, ESelectInfo::Type SelectType)
{
	UVariant* ParentVariant = ParentVariantPtr.Get();
	if (!NewItem.IsValid() || !Dependency || !ParentVariant)
	{
		return;
	}

	UVariantSet* DependencyVariantSet = Dependency->VariantSet.Get();
	if (!DependencyVariantSet)
	{
		return;
	}

	for (UVariant* Variant : DependencyVariantSet->GetVariants())
	{
		if (Variant && Variant->GetDisplayText().EqualTo(*NewItem))
		{
			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("VariantDependencyChanged", "Make variant '{0}' depend on variant '{0}'"),
				ParentVariant->GetDisplayText(),
				Variant->GetDisplayText()
			));

			ParentVariant->Modify();
			Dependency->Variant = Variant;
			return;
		}
	}
}

FText SDependencyRow::GetSelectedVariantSetOption() const
{
	if (Dependency)
	{
		if (UVariantSet* DependencyVariantSet = Dependency->VariantSet.Get())
		{
			// When we remove/delete a variant(set) in the editor, it remains referenced by the transaction buffer,
			// but we'll move it to the transient package, so here we check for that
			if (DependencyVariantSet->GetPackage() != GetTransientPackage())
			{
				return DependencyVariantSet->GetDisplayText();
			}
		}
	}

	return FText::FromString(TEXT("None"));
}

FText SDependencyRow::GetSelectedVariantOption() const
{
	if (Dependency)
	{
		if (UVariant* DependencyVariant = Dependency->Variant.Get())
		{
			if (DependencyVariant->GetPackage() != GetTransientPackage())
			{
				return DependencyVariant->GetDisplayText();
			}
		}
	}

	return FText::FromString(TEXT("None"));
}

FText SDependencyRow::GetDependentVariantSetText() const
{
	if (UVariant* ParentVariant = ParentVariantPtr.Get())
	{
		if (UVariantSet* ParentVariantSet = ParentVariant->GetParent())
		{
			return ParentVariantSet->GetDisplayText();
		}
	}

	return FText::FromString(TEXT("None"));
}

FText SDependencyRow::GetDependentVariantText() const
{
	if (UVariant* ParentVariant = ParentVariantPtr.Get())
	{
		return ParentVariant->GetDisplayText();
	}

	return FText::FromString(TEXT("None"));
}

void SDependencyRow::RebuildVariantSetOptions()
{
	VariantSetOptions.Reset();

	UVariant* ParentVariant = ParentVariantPtr.Get();
	if (!Dependency || !ParentVariant)
	{
		return;
	}

	ULevelVariantSets* LevelVariantSets = ParentVariant->GetTypedOuter<ULevelVariantSets>();
	if (!LevelVariantSets)
	{
		return;
	}

	UVariantSet* ParentVariantSet = ParentVariant->GetParent();

	VariantSetOptions.Reserve(LevelVariantSets->GetNumVariantSets());
	for (const UVariantSet* VariantSet : LevelVariantSets->GetVariantSets())
	{
		// A variant can't have its own variant set as a dependency
		if (VariantSet == ParentVariantSet)
		{
			continue;
		}

		// Check if this variant has anything we could pick as a dependency anyway
		bool bHasValidVariant = false;
		for ( const UVariant* Variant : VariantSet->GetVariants() )
		{
			if ( ParentVariant->IsValidDependency( Variant ) )
			{
				bHasValidVariant = true;
				break;
			}
		}
		if ( !bHasValidVariant )
		{
			continue;
		}

		VariantSetOptions.Add(MakeShared<FText>(VariantSet->GetDisplayText()));
	}
}

void SDependencyRow::RebuildVariantOptions()
{
	VariantOptions.Reset();

	UVariant* ParentVariant = ParentVariantPtr.Get();
	if (!Dependency || !ParentVariant)
	{
		return;
	}

	UVariantSet* VariantSet = Dependency->VariantSet.Get();
	if (!VariantSet)
	{
		return;
	}

	VariantOptions.Reserve(VariantSet->GetNumVariants());
	for (const UVariant* Variant : VariantSet->GetVariants())
	{
		if (!ParentVariant->IsValidDependency(Variant))
		{
			continue;
		}

		VariantOptions.Add(MakeShared<FText>(Variant->GetDisplayText()));
	}
}

FReply SDependencyRow::OnEnableRowToggled()
{
	UVariant* ParentVariant = ParentVariantPtr.Get();
	if (Dependency && ParentVariant)
	{
		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("ToggleDependencyTransaction", "Toggle a dependency for variant '{0}'"),
			ParentVariant->GetDisplayText()
		));

		ParentVariant->Modify();
		Dependency->bEnabled = !Dependency->bEnabled;
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
