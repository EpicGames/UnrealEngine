// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
class UVariant;

struct FColumnSizeData;
struct FVariantDependency;

// Adapter so that we can use arrays of these objects on SListViews and still reference the original Dependency
struct FVariantDependencyModel : TSharedFromThis<FVariantDependencyModel>
{
	FVariantDependencyModel(
		TWeakObjectPtr<UVariant> InParentVariant,
		FVariantDependency* InDependency,
		bool bInIsDependent,
		bool bInIsDivider)
		: ParentVariant(InParentVariant)
		, Dependency(InDependency)
		, bIsDependent(bInIsDependent)
		, bIsDivider(bInIsDivider)
	{}

	TWeakObjectPtr<UVariant> ParentVariant;
	FVariantDependency* Dependency = nullptr;
	bool bIsDependent = false;
	bool bIsDivider = false;
};

using FVariantDependencyModelPtr = TSharedPtr<FVariantDependencyModel>;

class SDependencyRow : public SMultiColumnTableRow<FVariantDependencyModelPtr>
{
public:
	SLATE_BEGIN_ARGS(SDependencyRow) {}
	SLATE_END_ARGS()

	static const FName VisibilityColumn;
	static const FName VariantSetColumn;
	static const FName VariantColumn;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FVariantDependencyModelPtr InDependencyModel, bool bInteractionEnabled);
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:
	void OnSelectedVariantSetChanged(TSharedPtr<FText> NewItem, ESelectInfo::Type SelectType);
	void OnSelectedVariantChanged(TSharedPtr<FText> NewItem, ESelectInfo::Type SelectType);

	FText GetSelectedVariantSetOption() const;
	FText GetSelectedVariantOption() const;

	FText GetDependentVariantSetText() const;
	FText GetDependentVariantText() const;

	void RebuildVariantSetOptions();
	void RebuildVariantOptions();

	FReply OnEnableRowToggled();
	EVisibility OnGetEyeIconVisibility() const;

private:
	TArray<TSharedPtr<FText>> VariantSetOptions;
	TArray<TSharedPtr<FText>> VariantOptions;

	TWeakObjectPtr<UVariant> ParentVariantPtr;
	FVariantDependency* Dependency = nullptr;

	bool bIsDivider = false;
	bool bIsDependent = false;
};
