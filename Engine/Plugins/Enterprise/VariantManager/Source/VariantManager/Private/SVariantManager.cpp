// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariantManager.h"

#include "ConsoleVariables.h"
#include "Components/StaticMeshComponent.h"
#include "DisplayNodes/VariantManagerActorNode.h"
#include "DisplayNodes/VariantManagerColorPropertyNode.h"
#include "DisplayNodes/VariantManagerEnumPropertyNode.h"
#include "DisplayNodes/VariantManagerFunctionPropertyNode.h"
#include "DisplayNodes/VariantManagerOptionPropertyNode.h"
#include "DisplayNodes/VariantManagerStringPropertyNode.h"
#include "DisplayNodes/VariantManagerStructPropertyNode.h"
#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DisplayNodes/VariantManagerVariantSetNode.h"
#include "Framework/Views/TableViewMetadata.h"
#include "FunctionCaller.h"
#include "IDesktopPlatform.h"
#include "LevelVariantSets.h"
#include "ISceneOutliner.h"
#include "PropertyValueOption.h"
#include "SDependencyRow.h"
#include "SVariantManagerActorListView.h"
#include "SVariantManagerNodeTreeView.h"
#include "SVariantManagerTableRow.h"
#include "Selection.h"
#include "SwitchActor.h"
#include "UObject/TextProperty.h"
#include "VariantManager.h"
#include "VariantManagerClipboard.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerLog.h"
#include "VariantManagerNodeTree.h"
#include "VariantManagerSelection.h"
#include "VariantManagerStyle.h"
#include "VariantManagerUtils.h"

#include "DesktopPlatformModule.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LevelEditor.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/TransactionObjectEvent.h"
#include "SceneOutlinerModule.h"
#include "ActorTreeItem.h"
#include "AssetViewUtils.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "PropertyCustomizationHelpers.h"
#include "SAssetDropTarget.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "Engine/Texture2D.h"
#include "Settings/EditorStyleSettings.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SVariantManager"

#define VM_COMMON_PADDING 3.0f
#define VM_COMMON_HEADER_MAX_HEIGHT 26.0f

FSplitterValues::FSplitterValues(FString& InSerialized)
{
	TArray<FString> SplitString;
	InSerialized.ParseIntoArray(SplitString, TEXT(";"));

	if (SplitString.Num() != 3)
	{
		return;
	}

	VariantColumn = FCString::Atof(*SplitString[0]);
	PropertyNameColumn = FCString::Atof(*SplitString[1]);
	PropertyValueColumn = FCString::Atof(*SplitString[2]);
}

FString FSplitterValues::ToString()
{
	return FString::SanitizeFloat(VariantColumn) + TEXT(";") +
		   FString::SanitizeFloat(PropertyNameColumn) + TEXT(";") +
		   FString::SanitizeFloat(PropertyValueColumn);
}

TSharedRef<SWidget> SVariantManager::MakeAddButton()
{
	return SNew(SPositiveActionButton)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("VariantSetPlusText", "Variant Set"))
		.OnClicked(this, &SVariantManager::OnAddVariantSetClicked);
}

TSharedRef<SWidget> SVariantManager::GetMenuContextForLevelVariantSets()
{
	FMenuBuilder Builder(true, MakeShared<FUICommandList>());

	FAssetData AssetData;
	if (TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin())
	{
		AssetData = FAssetData(VariantManager->GetCurrentLevelVariantSets());	
	}

	Builder.AddWidget(PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
		AssetData,
		false,
		{ULevelVariantSets::StaticClass()},
		{},
		FOnShouldFilterAsset(),
		FOnAssetSelected::CreateLambda([this](const FAssetData& AssetData)
			{
				if (AssetData.IsValid())
				{
					AssetViewUtils::OpenEditorForAsset(AssetData.GetAsset());
				}
			}),
		FSimpleDelegate()),
		FText::GetEmpty());

	return Builder.MakeWidget();
}

TSharedRef<SWidget> SVariantManager::MakeLevelVariantSetsComboButton()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 2.0f, 8.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("SVariantManager", "Asset", "Asset:"))
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SAssetDropTarget)
			.OnAreAssetsAcceptableForDropWithReason_Static(&SVariantManager::OnAreAssetsAcceptableForVariantManager)
			.OnAssetsDropped(this, &SVariantManager::OnAssetDroppedForVariantManager)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &SVariantManager::GetMenuContextForLevelVariantSets)
				.ButtonContent()
				[
					SNew(SBorder)
					.BorderBackgroundColor(FSlateColor(FColor::Black))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f)
						[
							SNew(SImage)
							.Image(FVariantManagerStyle::Get().GetBrush("VariantManager.Icon.Small"))
							.ColorAndOpacity(FSlateColor::UseForeground())
							.DesiredSizeOverride(FVector2D(16, 16))
						]

						+SHorizontalBox::Slot()
						.FillWidth(1)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
							.Text_Lambda([this]() -> FText
							{
								TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
								if (VarMan.IsValid() && VarMan->GetCurrentLevelVariantSets())
								{
									return FText::FromName(VarMan->GetCurrentLevelVariantSets()->GetFName());
								}

								return NSLOCTEXT("SVariantManager", "NoAssetSelected", "No asset selected");
							})
			                .OverflowPolicy(GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis ? TOptional(ETextOverflowPolicy::MiddleEllipsis) : NullOpt)
						]
					]
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.WidthOverride(22.0f)
			.HeightOverride(22.0f)
			.ToolTipText(LOCTEXT( "UseButtonToolTipText", "Use Selected Asset from Content Browser"))
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.OnClicked_Static(&SVariantManager::OnUseSelectedAssetFromContentBrowserForCurrentLevelVariantSet)
				.ContentPadding(0.0f)
				.IsFocusable(true)
				[ 
					SNew( SImage )
					.Image(FAppStyle::GetBrush("Icons.Use"))
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.WidthOverride(22.0f)
			.HeightOverride(22.0f)
			.ToolTipText(LOCTEXT( "BrowseButtonToolTipText", "Browse to Asset in Content Browser"))
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.OnClicked( this, &SVariantManager::OnBrowseToCurrentLevelVariantSet)
				.ContentPadding(0.0f)
				.IsFocusable(true)
				[ 
					SNew( SImage )
					.Image(FAppStyle::Get().GetBrush("Icons.BrowseContent"))
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			]
		];
}

TSharedRef<ITableRow> SVariantManager::MakeCapturedPropertyRow(TSharedPtr<FVariantManagerPropertyNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVariantManagerTableRow, OwnerTable, StaticCastSharedPtr<FVariantManagerDisplayNode>(Item).ToSharedRef());
}

TSharedRef<ITableRow> SVariantManager::MakeCapturedPropertyRowWithColumns(
	TSharedPtr<FVariantManagerPropertyNode> Item,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVariantManagerMultiColumnPropertyTableRow, OwnerTable, Item.ToSharedRef());
}

TSharedPtr<SWidget> SVariantManager::OnPropertyListContextMenuOpening() const
{
	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	CapturedPropertyListView->GetSelectedItems(SelectedNodes);

	if (SelectedNodes.Num() > 0)
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetPropertyListCommandBindings());

		SelectedNodes[0]->BuildContextMenu(MenuBuilder);

		return MenuBuilder.MakeWidget();
	}

	return nullptr;
}

void SVariantManager::OnOutlinerNodeSelectionChanged()
{
	RefreshActorList();
}

void SVariantManager::Construct(const FArguments& InArgs, TSharedRef<FVariantManager> InVariantManager)
{
	VariantManagerPtr = InVariantManager;

	bAutoCaptureProperties = false;

	HeaderStyle = &FAppStyle::Get().GetWidgetStyle<FTableColumnHeaderStyle>("TableView.Header.Column");
	SplitterStyle = &FAppStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter");
	
	CreateCommandBindings();

	SAssignNew(NodeTreeView, SVariantManagerNodeTreeView, InVariantManager->GetNodeTree());

	SAssignNew(ActorListView, SVariantManagerActorListView, InVariantManager)
		.ListItemsSource(&DisplayedActors);

	FString SplitterValuesString;
	if (GConfig->GetString(TEXT("VariantManager"), TEXT("SplitterValues"), SplitterValuesString, GEditorPerProjectIni))
	{
		SplitterValues = FSplitterValues(SplitterValuesString);
	}

	PropertiesNameColumnWidth = SplitterValues.PropertyNameColumn;
	PropertiesValueColumnWidth = 1.0f - SplitterValues.PropertyNameColumn;

	PropertiesColumnSizeData.NameColumnWidth = TAttribute<float>( this, &SVariantManager::OnGetPropertiesNameColumnWidth );
	PropertiesColumnSizeData.ValueColumnWidth = TAttribute<float>( this, &SVariantManager::OnGetPropertiesValueColumnWidth );
	PropertiesColumnSizeData.OnSplitterNameColumnChanged = SSplitter::FOnSlotResized::CreateLambda( [ this ]( float InNewWidth )
	{
		PropertiesNameColumnWidth = InNewWidth;
	});
	PropertiesColumnSizeData.OnSplitterValueColumnChanged = SSplitter::FOnSlotResized::CreateLambda([this](float InNewWidth)
	{
		PropertiesValueColumnWidth = InNewWidth;
	});

	InVariantManager->GetSelection().GetOnOutlinerNodeSelectionChanged().AddSP(this, &SVariantManager::OnOutlinerNodeSelectionChanged);
	InVariantManager->GetSelection().GetOnActorNodeSelectionChanged().AddSP(this, &SVariantManager::OnActorNodeSelectionChanged);

	UVariant::OnDependenciesUpdated.AddSP(this, &SVariantManager::OnVariantDependenciesUpdated);

	// Subscribe to when objects are modified so that we can auto-resolve when components/array properties are added/removed/renamed
	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &SVariantManager::OnObjectTransacted);
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SVariantManager::OnObjectPropertyChanged);
	OnPreObjectPropertyChangedHandle = FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &SVariantManager::OnPreObjectPropertyChanged);

	OnBeginPieHandle = FEditorDelegates::BeginPIE.AddRaw(this, &SVariantManager::OnPieEvent);
	OnEndPieHandle = FEditorDelegates::EndPIE.AddRaw(this, &SVariantManager::OnPieEvent);

	OnEditorSelectionChangedHandle = USelection::SelectionChangedEvent.AddRaw(this, &SVariantManager::OnEditorSelectionChanged);

	// Do this so that if we recompile a function caller changing a function name we'll rebuild our nodes to display the
	// new names
	OnBlueprintCompiledHandle = GEditor->OnBlueprintCompiled().AddSP(this, &SVariantManager::OnBlueprintCompiled);

	OnVariantThumbnailUpdatedHandle = UVariant::OnThumbnailUpdated.AddLambda([this](UVariant* Variant)
	{
		OnThumbnailChanged(Variant);
	});

	OnVariantSetThumbnailUpdatedHandle = UVariantSet::OnThumbnailUpdated.AddLambda([this](UVariantSet* VariantSet)
	{
		OnThumbnailChanged(VariantSet);
	});

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		OnMapChangedHandle = LevelEditorModule.OnMapChanged().AddSP(this, &SVariantManager::OnMapChanged);
	}

	RightTreeRootItems.Empty();
	RightTreeRootItems.Add(MakeShared<ERightTreeRowType>(ERightTreeRowType::DependenciesHeader));

	TSharedPtr<SHeaderRow> CollapsedHeader = SNew( SHeaderRow ).Visibility( EVisibility::Collapsed );

	TSharedRef< STreeView<TSharedRef<ERightTreeRowType>> > RightTree = SNew(STreeView<TSharedRef<ERightTreeRowType>>)
		.TreeItemsSource( &RightTreeRootItems )
		.SelectionMode( ESelectionMode::None )
		.HeaderRow( CollapsedHeader )
		.AllowOverscroll( EAllowOverscroll::No )
		.IsEnabled_Lambda([this]()
		{
			TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
			if (PinnedVariantManager.IsValid())
			{
				return PinnedVariantManager->GetSelection().GetSelectedOutlinerNodes().Num() > 0;
			}
			return false;
		})
		.OnGenerateRow( this, &SVariantManager::GenerateRightTreeRow )
		.OnGetChildren_Lambda([this]( TSharedRef<ERightTreeRowType> InRowType, TArray<TSharedRef<ERightTreeRowType>>& OutChildren )
		{
			OutChildren.Empty();
			if ( *InRowType == ERightTreeRowType::PropertiesHeader )
			{
				OutChildren.Add( MakeShared<ERightTreeRowType>( ERightTreeRowType::PropertiesContent ) );
			}
			else if ( *InRowType == ERightTreeRowType::DependenciesHeader )
			{
				OutChildren.Add( MakeShared<ERightTreeRowType>( ERightTreeRowType::DependenciesContent ) );
			}
		});

	// Expand right tree by default
	for ( const TSharedRef<ERightTreeRowType>& Item : RightTreeRootItems )
	{
		RightTree->SetItemExpansion(Item, true);
	}

	const float BorderThickness = FVariantManagerStyle::Get().GetFloat("VariantManager.Spacings.BorderThickness");;

	const bool bUseNewAssetPicker = UE::VariantManager::Get_CVar_VariantManager_AssetPicker().GetValueOnGameThread();
	TSharedRef<SWidget> VariantSetsComboBox = bUseNewAssetPicker ? MakeLevelVariantSetsComboButton() : SNullWidget::NullWidget;
	TSharedRef<SWidget> OldSearchBox = bUseNewAssetPicker ? SNullWidget::NullWidget : SNew(SSearchBox)
		.HintText(LOCTEXT("VariantManagerFilterText", "Search Variant"))
		.OnTextChanged(this, &SVariantManager::OnOutlinerSearchChanged)
		.MinDesiredWidth(200);
	TSharedRef<SWidget> NewSearchBox = !bUseNewAssetPicker ? SNullWidget::NullWidget : SNew(SSearchBox)
		.HintText(LOCTEXT("VariantManagerFilterText", "Search Variant"))
		.OnTextChanged(this, &SVariantManager::OnOutlinerSearchChanged)
		.MinDesiredWidth(200);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Main header bar (+ Variant Set button and etc)
		+SVerticalBox::Slot()
		.MaxHeight(VM_COMMON_HEADER_MAX_HEIGHT)
		.AutoHeight()
		.Padding(FMargin(VM_COMMON_PADDING, VM_COMMON_PADDING, 0.0f, VM_COMMON_PADDING))
		[
			SNew(SBox)
			.HeightOverride(VM_COMMON_HEADER_MAX_HEIGHT)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, VM_COMMON_PADDING+2.0f, 1.f))
				.AutoWidth()
				[
					VariantSetsComboBox
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, VM_COMMON_PADDING+2.0f, 1.f))
				.AutoWidth()
				[
					MakeAddButton()
				]
					
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					OldSearchBox
				]
				
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(FMargin(0.f, 0.f, VM_COMMON_PADDING+2.0f, 0.f))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.Style(& FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
						.ToolTipText(LOCTEXT("AutoCaptureTooltip", "Enable or disable auto-capturing properties"))
						.IsChecked_Lambda([&bAutoCaptureProperties = bAutoCaptureProperties]()
						{
							return bAutoCaptureProperties ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([&bAutoCaptureProperties = bAutoCaptureProperties](const ECheckBoxState NewState)
						{
							bAutoCaptureProperties = NewState == ECheckBoxState::Checked;
						})
						[
							SNew(SBox)
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FVariantManagerStyle::Get().GetBrush("VariantManager.AutoCapture.Icon"))
								]

								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.Padding(8.f, 0.5f, 0.f, 0.f)
								.AutoWidth()
								[
									SNew(STextBlock)
									.TextStyle(&FAppStyle::Get().GetWidgetStyle< FTextBlockStyle >("SmallButtonText"))
									.Justification(ETextJustify::Center)
									.Text(LOCTEXT("VariantManagerAutoCapture", "Auto Capture"))
								]
							]
						]
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Horizontal)
			.Thickness(3)
			.SeparatorImage(&SplitterStyle->HandleNormalBrush)
		]
		
		+ SVerticalBox::Slot()
		.VAlign( VAlign_Fill )
		.FillHeight( 1.0f )
		[
			SNew(SBox)
			[
				SAssignNew(MainSplitter, SSplitter)
				.Orientation(Orient_Horizontal)
				.PhysicalSplitterHandleSize(BorderThickness)
				.HitDetectionSplitterHandleSize(BorderThickness)

				// VariantSet/Variant column
				+SSplitter::Slot()
				.Value(SplitterValues.VariantColumn)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(10, 7, 8, 8))
					[
						NewSearchBox
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SScrollBorder, NodeTreeView.ToSharedRef())
						[
							SNew(SBox) // Very important to prevent the tree from expanding freely
							[
								NodeTreeView.ToSharedRef()
							]
						]
					]
				]

				// Properties/Dependencies column
				+ SSplitter::Slot()
				.Value( 1.0f - SplitterValues.VariantColumn )
				[
					SNew(SSplitter)
					.Orientation(Orient_Vertical)
					.PhysicalSplitterHandleSize(BorderThickness)
					.HitDetectionSplitterHandleSize(BorderThickness)
					
					+ SSplitter::Slot()
					[
						GenerateRightTreePropertiesRowContent()
					]
					
					+ SSplitter::Slot()
					[
						RightTree
					]
				]
			]
		]
	];

	SortItems();

	RefreshVariantTree();
}

SVariantManager::~SVariantManager()
{
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);

	GEditor->OnBlueprintCompiled().Remove(OnBlueprintCompiledHandle);
	OnBlueprintCompiledHandle.Reset();

	FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
	OnObjectTransactedHandle.Reset();

	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
	OnObjectPropertyChangedHandle.Reset();

	FCoreUObjectDelegates::OnPreObjectPropertyChanged.Remove(OnPreObjectPropertyChangedHandle);
	OnPreObjectPropertyChangedHandle.Reset();

	FEditorDelegates::BeginPIE.Remove(OnBeginPieHandle);
	OnBeginPieHandle.Reset();

	FEditorDelegates::EndPIE.Remove(OnEndPieHandle);
	OnEndPieHandle.Reset();

	USelection::SelectionChangedEvent.Remove(OnEditorSelectionChangedHandle);
	OnEditorSelectionChangedHandle.Reset();

	UVariant::OnDependenciesUpdated.RemoveAll(this);

	UVariant::OnThumbnailUpdated.Remove(OnVariantThumbnailUpdatedHandle);
	OnVariantThumbnailUpdatedHandle.Reset();

	UVariantSet::OnThumbnailUpdated.Remove(OnVariantSetThumbnailUpdatedHandle);
	OnVariantSetThumbnailUpdatedHandle.Reset();

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditorModule.OnMapChanged().Remove(OnMapChangedHandle);
		OnMapChangedHandle.Reset();
	}

	// Save splitter layout
	if ( MainSplitter )
	{
		FChildren* MainSlots = MainSplitter->GetChildren();
		if (MainSlots->Num() > 0)
		{
			FSplitterValues Values;
			Values.VariantColumn = MainSplitter->SlotAt(0).GetSizeValue();
			Values.PropertyNameColumn = OnGetPropertiesNameColumnWidth();
			Values.PropertyValueColumn = OnGetPropertiesValueColumnWidth();

			GConfig->SetString(TEXT("VariantManager"), TEXT("SplitterValues"), *Values.ToString(), GEditorPerProjectIni);
		}
	}
}

void SVariantManager::CreateCommandBindings()
{
	VariantTreeCommandBindings = MakeShareable(new FUICommandList);
	ActorListCommandBindings = MakeShareable(new FUICommandList);
	PropertyListCommandBindings = MakeShareable(new FUICommandList);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SVariantManager::CutSelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCutVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SVariantManager::CopySelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCopyVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SVariantManager::PasteSelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanPasteVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SVariantManager::DeleteSelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanDeleteVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SVariantManager::DuplicateSelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanDuplicateVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SVariantManager::RenameSelectionVariantTree),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRenameVariantTree)
	);

	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().AddVariantSetCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::CreateNewVariantSet),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCreateNewVariantSet)
	);

	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().SwitchOnSelectedVariantCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::SwitchOnSelectedVariant),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanSwitchOnVariant)
	);

	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().CreateThumbnailCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::CreateThumbnail),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCreateThumbnail)
	);
	
	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().SelectThumbnailAssetCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::SelectThumbnailAsset),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanLoadThumbnail)
	);

	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().LoadThumbnailCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::LoadThumbnail),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanLoadThumbnail)
	);

	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().ClearThumbnailCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::ClearThumbnail),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanClearThumbnail)
	);

	// This command is added to both lists so that we can add actors by right clicking on variant
	// nodes or by right clicking on the actor list with a variant node selected
	VariantTreeCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().AddSelectedActorsCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::AddEditorSelectedActorsToVariant),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanAddEditorSelectedActorsToVariant)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SVariantManager::CutSelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCutActorList)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SVariantManager::CopySelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCopyActorList)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SVariantManager::PasteSelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanPasteActorList)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SVariantManager::DeleteSelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanDeleteActorList)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SVariantManager::DuplicateSelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanDuplicateActorList)
	);

	ActorListCommandBindings->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SVariantManager::RenameSelectionActorList),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRenameActorList)
	);

	// This command is added to both lists so that we can add actors by right clicking on variant
	// nodes or by right clicking on the actor list with a variant node selected
	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().AddSelectedActorsCommand,
		FExecuteAction::CreateSP(this, &SVariantManager::AddEditorSelectedActorsToVariant),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanAddEditorSelectedActorsToVariant)
	);

	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().AddPropertyCaptures,
		FExecuteAction::CreateSP(this, &SVariantManager::CaptureNewPropertiesFromSelectedActors),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCaptureNewPropertiesFromSelectedActors)
	);

	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().AddFunction,
		FExecuteAction::CreateSP(this, &SVariantManager::AddFunctionCaller),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanAddFunctionCaller)
	);

	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RebindActorDisabled,
		FExecuteAction::CreateLambda([](){}),
		FCanExecuteAction::CreateLambda([](){ return false; })
	);

	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RebindToSelected,
		FExecuteAction::CreateSP(this, &SVariantManager::RebindToSelectedActor),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRebindToSelectedActor)
	);

	ActorListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RemoveActorBindings,
		FExecuteAction::CreateSP(this, &SVariantManager::RemoveActorBindings),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRemoveActorBindings)
	);

	PropertyListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().ApplyProperty,
		FExecuteAction::CreateSP(this, &SVariantManager::ApplyProperty),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanApplyProperty)
	);

	PropertyListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RecordProperty,
		FExecuteAction::CreateSP(this, &SVariantManager::RecordProperty),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRecordProperty)
	);

	PropertyListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RemoveCapture,
		FExecuteAction::CreateSP(this, &SVariantManager::RemoveCapture),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRemoveCapture)
	);

	PropertyListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().CallFunction,
		FExecuteAction::CreateSP(this, &SVariantManager::CallDirectorFunction),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanCallDirectorFunction)
	);

	PropertyListCommandBindings->MapAction(
		FVariantManagerEditorCommands::Get().RemoveFunction,
		FExecuteAction::CreateSP(this, &SVariantManager::RemoveDirectorFunctionCaller),
		FCanExecuteAction::CreateSP(this, &SVariantManager::CanRemoveDirectorFunctionCaller)
	);
}

void SVariantManager::AddEditorSelectedActorsToVariant()
{
	TArray<AActor*> Actors;
	USelection* Selection = GEditor->GetSelectedActors();
	for (FSelectionIterator SelectedObject(*Selection); SelectedObject; ++SelectedObject)
	{
		if (AActor* SelectedActor = Cast<AActor>(*SelectedObject))
		{
			Actors.Add(SelectedActor);
		}
	}

	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>>& Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	TArray<UVariant*> SelectedVariants;
	for (const TSharedRef<FVariantManagerDisplayNode>& Node : Nodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> SomeNodeAsVariant = StaticCastSharedRef<FVariantManagerVariantNode>(Node);

			if (SomeNodeAsVariant.IsValid())
			{
				SelectedVariants.Add(&SomeNodeAsVariant->GetVariant());
			}
		}
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("AddEditorSelectedActorsToVariantTransaction", "Add {0} actor {0}|plural(one=binding,other=bindings) to {1} {1}|plural(one=variant,other=variants)"),
		Actors.Num(),
		SelectedVariants.Num()
	));

	VariantManager->CreateObjectBindingsAndCaptures(Actors, SelectedVariants);

	RefreshActorList();
}

bool SVariantManager::CanAddEditorSelectedActorsToVariant()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>>& Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	// Get all selected variants
	TArray<UVariant*> SelectedVariants;
	for (const TSharedRef<FVariantManagerDisplayNode>& Node : Nodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> SomeNodeAsVariant = StaticCastSharedRef<FVariantManagerVariantNode>(Node);
			if (SomeNodeAsVariant.IsValid())
			{
				SelectedVariants.Add(&SomeNodeAsVariant->GetVariant());
			}
		}
	}

	// Get actors selected in the editor
	TArray<TWeakObjectPtr<AActor>> SelectedActors;
	USelection* Selection = GEditor->GetSelectedActors();
	for (FSelectionIterator SelectedObject(*Selection); SelectedObject; ++SelectedObject)
	{
		AActor* SelectedActor = Cast<AActor>(*SelectedObject);
		if (SelectedActor)
		{
			SelectedActors.Add(SelectedActor);
		}
	}

	// See if we can add at least one new binding to at least one of the selected variants
	for (UVariant* Var : SelectedVariants)
	{
		TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;
		VariantManager->CanAddActorsToVariant(SelectedActors, Var, ActorsWeCanAdd);

		if (ActorsWeCanAdd.Num() > 0)
		{
			return true;
		}
	}
	return false;
}

void SVariantManager::CreateNewVariantSet()
{
	TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
	if (VarMan.IsValid())
	{
		VarMan->CreateVariantSet(VarMan->GetCurrentLevelVariantSets());

		RefreshVariantTree();
	}
}

bool SVariantManager::CanCreateNewVariantSet()
{
	return true;
}

// Used both by copy and cut
void CopySelectionVariantTreeInternal(TSharedPtr<FVariantManager> VariantManager, TArray<UVariant*>& InCopiedVariants, TArray<UVariantSet*>& InCopiedVariantSets)
{
	FVariantManagerClipboard::Empty();

	// Keep track of variant duplication so that we can transfer thumbnails later
	// We could use the clipboard arrays for this, but this does not make
	// any assumptions about how the clipboard stores its stuff
	TArray<UVariant*> OriginalVariants;
	TArray<UVariant*> NewVariants;

	// Add copies of our stuff to the clipboard
	for (UVariantSet* VariantSet : InCopiedVariantSets)
	{
		UVariantSet* NewVariantSet = DuplicateObject(VariantSet, nullptr);
		FVariantManagerClipboard::Push(NewVariantSet);

		OriginalVariants.Append(VariantSet->GetVariants());
		NewVariants.Append(NewVariantSet->GetVariants());
	}
	for (UVariant* Variant : InCopiedVariants)
	{
		// Don't copy variants those parents are also copied
		if (InCopiedVariantSets.Contains(Variant->GetParent()))
		{
			continue;
		}

		// Transient package here because our Outer might be deleted while we're in the clipboard
		UVariant* NewVariant = DuplicateObject(Variant, nullptr);
		FVariantManagerClipboard::Push(NewVariant);

		OriginalVariants.Add(Variant);
		NewVariants.Add(NewVariant);
	}

	VariantManager->CopyVariantThumbnails(NewVariants, OriginalVariants);
}

void SVariantManager::CutSelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> CopiedVariants;
	TArray<UVariantSet*> CopiedVariantSets;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(CopiedVariants, CopiedVariantSets);

	CopySelectionVariantTreeInternal(VariantManager, CopiedVariants, CopiedVariantSets);

	// Don't capture CopySelection in the transaction buffer because if we undo we kind of expect
	// our cut stuff to still be in the clipboard
	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("CutSelectionVariantTreeTransaction", "Cut {0} {0}|plural(one=variant,other=variants) and {1} variant {1}|plural(one=set,other=sets)"),
		CopiedVariants.Num(),
		CopiedVariantSets.Num()
		));


	VariantManager->RemoveVariantsFromParent(CopiedVariants);
	VariantManager->RemoveVariantSetsFromParent(CopiedVariantSets);

	RefreshVariantTree();
	RefreshActorList();
}

void SVariantManager::CopySelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> CopiedVariants;
	TArray<UVariantSet*> CopiedVariantSets;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(CopiedVariants, CopiedVariantSets);

	CopySelectionVariantTreeInternal(VariantManager, CopiedVariants, CopiedVariantSets);
}

void SVariantManager::PasteSelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	const TSet<TSharedRef<FVariantManagerDisplayNode>> SelectedNodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();
	ULevelVariantSets* LevelVarSets = VariantManager->GetCurrentLevelVariantSets();

	// Keep track of variant duplication so that we can transfer thumbnails later
	// We could use the clipboard arrays for this, but this does not make
	// any assumptions about how the clipboard stores its stuff
	TArray<UVariant*> OriginalVariants;
	TArray<UVariant*> NewVariants;

	const TArray<TStrongObjectPtr<UVariantSet>>& CopiedVariantSets = FVariantManagerClipboard::GetVariantSets();
	const TArray<TStrongObjectPtr<UVariant>>& CopiedVariants = FVariantManagerClipboard::GetVariants();

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("PasteSelectionVariantTreeTransaction", "Paste {0} {0}|plural(one=variant,other=variants) and {1} variant {1}|plural(one=set,other=sets)"),
		CopiedVariants.Num(),
		CopiedVariantSets.Num()
		));

	// Paste variant sets onto the tree, regardless of where we clicked
	TArray<UVariantSet*> VarSetsToAdd;
	for (const TStrongObjectPtr<UVariantSet>& CopiedVarSet : CopiedVariantSets)
	{
		// Duplicate objects since we'll maintain this in the clipboard
		UVariantSet* NewVariantSet = DuplicateObject(CopiedVarSet.Get(), nullptr);
		VarSetsToAdd.Add(NewVariantSet);

		OriginalVariants.Append(CopiedVarSet.Get()->GetVariants());
		NewVariants.Append(NewVariantSet->GetVariants());
	}
	VariantManager->AddVariantSets(VarSetsToAdd, LevelVarSets);

	// Add our copied variants to either the first varset we find, or create a new one
	if (CopiedVariants.Num() > 0)
	{
		TSharedPtr<FVariantManagerVariantSetNode> FirstVarSetNodeWeFound(nullptr);

		// See if we have a variant set node selected
		for (const TSharedRef<FVariantManagerDisplayNode>& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->GetType() == EVariantManagerNodeType::VariantSet)
			{
				FirstVarSetNodeWeFound = StaticCastSharedRef<FVariantManagerVariantSetNode>(SelectedNode);
				if (FirstVarSetNodeWeFound.IsValid())
				{
					break;
				}
			}
		}

		// If not, but we have selected a variant, pick its variant set so that we can paste
		// the copied variants as siblings
		for (const TSharedRef<FVariantManagerDisplayNode>& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->GetType() == EVariantManagerNodeType::Variant)
			{
				TSharedPtr<FVariantManagerVariantNode> SomeVariantNode = StaticCastSharedRef<FVariantManagerVariantNode>(SelectedNode);
				if (SomeVariantNode.IsValid())
				{
					FirstVarSetNodeWeFound = StaticCastSharedPtr<FVariantManagerVariantSetNode>(SomeVariantNode->GetParent());
				}
			}
		}

		UVariantSet* TargetVarSet = nullptr;

		// If we still have nowhere to place our copied variants, create a new variant set
		if (FirstVarSetNodeWeFound.IsValid())
		{
			TargetVarSet = &FirstVarSetNodeWeFound->GetVariantSet();
		}
		if (TargetVarSet == nullptr)
		{
			TargetVarSet = VariantManager->CreateVariantSet(LevelVarSets);
		}

		// Actually paste our copied variants
		TArray<UVariant*> VariantsToAdd;
		for (const TStrongObjectPtr<UVariant>& CopiedVariant : CopiedVariants)
		{
			// Make sure that if we pasted our parent variant set (which will already have CopiedVariant), we
			// don't do it again. We do this check on copy/cut, but its better to be safe
			UVariantSet* ParentVariantSet = CopiedVariant.Get()->GetParent();
			if (CopiedVariantSets.FindByPredicate([ParentVariantSet](const TStrongObjectPtr<UVariantSet>& VarSet)
			{
				return VarSet.Get() == ParentVariantSet;
			}))
			{
				continue;
			}

			// Duplicate objects since we'll maintain this in the clipboard
			UVariant* NewVariant = DuplicateObject(CopiedVariant.Get(), nullptr);
			VariantsToAdd.Add(NewVariant);

			OriginalVariants.Add(CopiedVariant.Get());
			NewVariants.Add(NewVariant);
		}
		VariantManager->AddVariants(VariantsToAdd, TargetVarSet);
	}

	VariantManager->CopyVariantThumbnails(NewVariants, OriginalVariants);

	RefreshVariantTree();
	RefreshActorList(); // For example if we paste a variant within an empty, selected variant set. We need to show the actors of the new variant
}

void SVariantManager::DeleteSelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> VariantsToDelete;
	TArray<UVariantSet*> VariantSetsToDelete;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(VariantsToDelete, VariantSetsToDelete);

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("DeleteSelectionVariantTreeTransaction", "Delete {0} {0}|plural(one=variant,other=variants) and {1} variant {1}|plural(one=set,other=sets)"),
		VariantsToDelete.Num(),
		VariantSetsToDelete.Num()
		));

	VariantManager->RemoveVariantsFromParent(VariantsToDelete);
	VariantManager->RemoveVariantSetsFromParent(VariantSetsToDelete);

	RefreshVariantTree();
	RefreshActorList();
}

void SVariantManager::DuplicateSelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	ULevelVariantSets* LevelVarSets = VariantManager->GetCurrentLevelVariantSets();

	// Collect all variants and variant sets that we selected
	TArray<UVariant*> VariantsToDuplicate;
	TArray<UVariantSet*> VariantSetsToDuplicate;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(VariantsToDuplicate, VariantSetsToDuplicate);

	// Keep track of variant duplication so that we can transfer thumbnails later
	// We could use the clipboard arrays for this, but this does not make
	// any assumptions about how the clipboard stores its stuff
	TArray<UVariant*> OriginalVariants;
	TArray<UVariant*> NewVariants;

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("DuplicateSelectionVariantTreeTransaction", "Duplicate {0} {0}|plural(one=variant,other=variants) and {1} variant {1}|plural(one=set,other=sets)"),
		VariantsToDuplicate.Num(),
		VariantSetsToDuplicate.Num()
		));

	// Duplicate variants
	for (UVariant* Variant : VariantsToDuplicate)
	{
		UVariantSet* ParentVariantSet = Variant->GetParent();
		if (VariantSetsToDuplicate.Contains(ParentVariantSet))
		{
			continue;
		}

		UVariant* NewVariant = DuplicateObject(Variant, nullptr);

		OriginalVariants.Add(Variant);
		NewVariants.Add(NewVariant);

		// Add individually because we might have different parents
		TArray<UVariant*> VariantsToAdd({NewVariant});
		VariantManager->AddVariants(VariantsToAdd, ParentVariantSet);
	}

	// Duplicate variant sets
	TArray<UVariantSet*> VarSetsToAdd;
	for (UVariantSet* VariantSet : VariantSetsToDuplicate)
	{
		UVariantSet* NewVariantSet = DuplicateObject(VariantSet, nullptr);

		OriginalVariants.Append(VariantSet->GetVariants());
		NewVariants.Append(NewVariantSet->GetVariants());

		VarSetsToAdd.Add(NewVariantSet);
	}
	VariantManager->AddVariantSets(VarSetsToAdd, LevelVarSets);

	VariantManager->CopyVariantThumbnails(NewVariants, OriginalVariants);

	RefreshVariantTree();
}

void SVariantManager::RenameSelectionVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	for (const TSharedRef<FVariantManagerDisplayNode>& SomeNode : Nodes)
	{
		SomeNode->StartRenaming();
	}
}

bool SVariantManager::CanCutVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	return Nodes.Num() > 0;
}

bool SVariantManager::CanCopyVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	return Nodes.Num() > 0;
}

bool SVariantManager::CanPasteVariantTree()
{
	return (FVariantManagerClipboard::GetVariants().Num() + FVariantManagerClipboard::GetVariantSets().Num()) > 0;
}

bool SVariantManager::CanDeleteVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	return Nodes.Num() > 0;
}

bool SVariantManager::CanDuplicateVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	return Nodes.Num() > 0;
}

bool SVariantManager::CanRenameVariantTree()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>> Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	for (const TSharedRef<FVariantManagerDisplayNode>& SomeNode : Nodes)
	{
		if (!SomeNode->IsReadOnly())
		{
			return true;
		}
	}

	return false;
}

void SVariantManager::CutSelectionActorList()
{

}

void SVariantManager::CopySelectionActorList()
{

}

void SVariantManager::PasteSelectionActorList()
{

}

void SVariantManager::DeleteSelectionActorList()
{

}

void SVariantManager::DuplicateSelectionActorList()
{

}

void SVariantManager::RenameSelectionActorList()
{

}

bool SVariantManager::CanCutActorList()
{
	return true;
}

bool SVariantManager::CanCopyActorList()
{
	return true;
}

bool SVariantManager::CanPasteActorList()
{
	return true;
}

bool SVariantManager::CanDeleteActorList()
{
	return true;
}

bool SVariantManager::CanDuplicateActorList()
{
	return true;
}

bool SVariantManager::CanRenameActorList()
{
	return true;
}

void SVariantManager::SwitchOnSelectedVariant()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedRef<FVariantManagerDisplayNode>> SelectedNodes = VariantManager->GetSelection().GetSelectedOutlinerNodes().Array();
	NodeTreeView->SortAsDisplayed(SelectedNodes);

	for (const TSharedRef<FVariantManagerDisplayNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> VarNode = StaticCastSharedRef<FVariantManagerVariantNode>(Node);

			UVariant* Variant = &VarNode->GetVariant();

			SwitchOnVariant(Variant);
		}
	}
}

void SVariantManager::CreateThumbnail()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> Vars;
	TArray<UVariantSet*> VarSets;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Vars, VarSets);
	if (Vars.Num() + VarSets.Num() < 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CaptureThumbnailTransaction", "Capture viewport as a thumbnail for a variant or variant set"));

	for (UVariant* Var : Vars)
	{
		Var->SetThumbnailFromEditorViewport();

		if (UVariantSet* VarSet = Var->GetParent())
		{
			UpdateVariantSetThumbnail(VarSet);
		}
	}
	for (UVariantSet* VarSet : VarSets)
	{
		VarSet->SetThumbnailFromEditorViewport();
	}
}

void SVariantManager::SelectThumbnailAsset()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> Vars;
	TArray<UVariantSet*> VarSets;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Vars, VarSets);
	if (Vars.Num() + VarSets.Num() < 1)
	{
		return;
	}

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	FOpenAssetDialogConfig Config;
	Config.bAllowMultipleSelection = false;
	Config.DialogTitleOverride = NSLOCTEXT("SVariantManager", "SelectAnAssetForThumbnail", "Select an Asset for Thumbnail");
	TArray<FAssetData> AssetDatas = ContentBrowser.CreateModalOpenAssetDialog(Config);
	if (AssetDatas.Num() == 1)
	{
		if (UObject* ObjectThumbnail = AssetDatas[0].GetAsset())
		{
			FScopedTransaction Transaction(LOCTEXT("SelectThumbnailAssetTransaction", "Select asset as a thumbnail for a variant or variant set"));

			for (UVariant* Var : Vars)
			{
				Var->SetThumbnailFromObject(ObjectThumbnail);

				if (UVariantSet* VarSet = Var->GetParent())
				{
					UpdateVariantSetThumbnail(VarSet);
				}
			}
			for (UVariantSet* VarSet : VarSets)
			{
				VarSet->SetThumbnailFromObject(ObjectThumbnail);
			}
		}
	}
}

void SVariantManager::LoadThumbnail()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (!VariantManager.IsValid())
	{
		return;
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform == nullptr)
	{
		return;
	}

	TArray<UVariant*> Vars;
	TArray<UVariantSet*> VarSets;
	VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Vars, VarSets);
	if (Vars.Num() + VarSets.Num() < 1)
	{
		return;
	}

	// Formats that IImageWrapper can handle
	const FString Filter = TEXT("Image files (*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns)|*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns|All files (*.*)|*.*");
	TArray<FString> OutFiles;
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

	bool bPickedFile = DesktopPlatform->OpenFileDialog(ParentWindowHandle,
													   LOCTEXT("LoadThumbnailFromFileTitle", "Choose a file for the thumbnail").ToString(),
													   TEXT(""), TEXT(""), Filter, EFileDialogFlags::None, OutFiles);
	if (!bPickedFile || OutFiles.Num() != 1)
	{
		return;
	}

	FString SourceImagePath = FPaths::ConvertRelativePathToFull(OutFiles[0]);

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("LoadThumbnailTransaction", "Load file '{0}' as a thumbnail for a variant or variant set"), FText::FromString(SourceImagePath)
	));

	for (UVariant* Var : Vars)
	{
		Var->SetThumbnailFromFile(SourceImagePath);

		if (UVariantSet* VarSet = Var->GetParent())
		{
			UpdateVariantSetThumbnail(VarSet);
		}
	}
	for (UVariantSet* VarSet : VarSets)
	{
		VarSet->SetThumbnailFromFile(SourceImagePath);
	}
}

void SVariantManager::ClearThumbnail()
{
	TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
	if (!VarMan.IsValid())
	{
		return;
	}

	TArray<UVariant*> SelectedVariants;
	TArray<UVariantSet*> SelectedVariantSets;
	VarMan->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
	if (SelectedVariants.Num() + SelectedVariantSets.Num() < 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ClearThumbnailTransaction", "Cleared variant thumbnails"));

	for (UVariant* Variant : SelectedVariants)
	{
		Variant->SetThumbnailFromTexture(nullptr);
	}
	for (UVariantSet* VariantSet : SelectedVariantSets)
	{
		VariantSet->SetThumbnailFromTexture(nullptr);
	}
}

void SVariantManager::UpdateVariantSetThumbnail(UVariantSet* InVariantSet)
{
	if (nullptr == InVariantSet || InVariantSet->GetThumbnail())
	{
		return; // VariantSet is null or already has a thumbnail
	}

	for (UVariant* Variant : InVariantSet->GetVariants())
	{
		if (UTexture2D* Thumbnail = Variant->GetThumbnail())
		{
			InVariantSet->SetThumbnailFromTexture(Thumbnail);
			break;
		}
	}
}

void SVariantManager::OnAssetDroppedForVariantManager(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas)
{
	if (AssetDatas.IsValidIndex(0))
	{
		const FAssetData& AssetData = AssetDatas[0];
		if (AssetData.AssetClassPath != FTopLevelAssetPath(ULevelVariantSets::StaticClass()))
		{
			return;
		}
		UObject* Asset = AssetData.GetAsset();
		ULevelVariantSets* LevelVariantSets = Cast<ULevelVariantSets>(Asset);
		if (!LevelVariantSets)
		{
			return;
		}
		AssetViewUtils::OpenEditorForAsset(LevelVariantSets);
	}
}

bool SVariantManager::OnAreAssetsAcceptableForVariantManager(TArrayView<FAssetData> Assets, FText& OutReason)
{
	if (Assets.Num() != 1)
	{
		OutReason = LOCTEXT("MultipleAssetsNotAllowed", "Please select only one asset.");
		return false;
	}

	FAssetData AssetData = Assets[0];

	if (AssetData.AssetClassPath != FTopLevelAssetPath(ULevelVariantSets::StaticClass()))
	{
		OutReason = LOCTEXT("LevelVariantSetsNotAllowed", "Only Level Variant Sets assets can be opened by Variant Manager.");
		return false;
	}

	return true;
}

FReply SVariantManager::OnUseSelectedAssetFromContentBrowserForCurrentLevelVariantSet()
{
	IContentBrowserSingleton& Singleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();;
	TArray<FAssetData> ContentBrowserSelectedAssets;
	Singleton.GetSelectedAssets(ContentBrowserSelectedAssets);

	if (ContentBrowserSelectedAssets.Num() == 1)
	{
		FAssetData SelectedAsset = ContentBrowserSelectedAssets[0];

		if (SelectedAsset.IsInstanceOf(ULevelVariantSets::StaticClass()))
		{
			if (ULevelVariantSets* LevelVariantSets = Cast<ULevelVariantSets>(SelectedAsset.GetAsset()))
			{
				AssetViewUtils::OpenEditorForAsset(LevelVariantSets);
			}
		}
	}

	return FReply::Handled();
}

FReply SVariantManager::OnBrowseToCurrentLevelVariantSet() const
{
	if (TSharedPtr<FVariantManager> VariantManger = VariantManagerPtr.Pin())
	{
		if (ULevelVariantSets* LevelVariantSets = VariantManger->GetCurrentLevelVariantSets())
		{
			TArray<FAssetData> AssetDataArray;
			AssetDataArray.Add(FAssetData(LevelVariantSets));
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(AssetDataArray);
		}
	}

	return FReply::Handled();
}

bool SVariantManager::CanSwitchOnVariant()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	const TSet<TSharedRef<FVariantManagerDisplayNode>>& Nodes = VariantManager->GetSelection().GetSelectedOutlinerNodes();

	int32 NumVariants = 0;

	for (const TSharedRef<FVariantManagerDisplayNode>& SomeNode : Nodes)
	{
		if (SomeNode->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> SomeNodeAsVariant = StaticCastSharedRef<FVariantManagerVariantNode>(SomeNode);

			if (SomeNodeAsVariant.IsValid())
			{
				NumVariants += 1;
			}
		}
	}

	return NumVariants >= 1;
}

bool SVariantManager::CanCreateThumbnail()
{
	return true;
}

bool SVariantManager::CanLoadThumbnail()
{
	return true;
}

bool SVariantManager::CanClearThumbnail()
{
	return true;
}

void SVariantManager::CaptureNewPropertiesFromSelectedActors()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

		FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
		const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();

		TArray<UVariantObjectBinding*> SelectedBindings;
		for (const TSharedRef<FVariantManagerActorNode>& ActorNode : SelectedActorNodes)
		{
			UVariantObjectBinding* Binding = ActorNode->GetObjectBinding().Get();
			if (Binding)
			{
				SelectedBindings.Add(Binding);
			}
		}

		int32 NumBindings = SelectedBindings.Num();
	if (NumBindings == 0)
	{
		return;
	}

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("ActorNodeCaptureNewPropertiesTransaction", "Capture new properties for {0} actor {0}|plural(one=binding,other=bindings)"),
			NumBindings
			));

		PinnedVariantManager->CaptureNewProperties(SelectedBindings);
		PinnedVariantManager->GetVariantManagerWidget()->RefreshPropertyList();
}

bool SVariantManager::CanCaptureNewPropertiesFromSelectedActors()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
		const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();

		return SelectedActorNodes.Num() > 0;
	}

	return false;
}

void SVariantManager::AddFunctionCaller()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
	const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();
	if (SelectedActorNodes.Num() < 1)
	{
		return;
	}

	int32 NumNewCallers = SelectedActorNodes.Num();
	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("AddFunctionCaller", "Created {0} new function {0}|plural(one=caller,other=callers)"),
		NumNewCallers
	));

	for (const TSharedPtr<FVariantManagerActorNode> Node : SelectedActorNodes)
	{
		UVariantObjectBinding* Binding = Node->GetObjectBinding().Get();
		if (Binding)
		{
			PinnedVariantManager->CreateFunctionCaller({Binding});
		}
	}

	RefreshPropertyList();
}

bool SVariantManager::CanAddFunctionCaller()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return false;
	}

	FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
	const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();

	return SelectedActorNodes.Num() > 0;
}

void SVariantManager::GetSelectedBindingAndEditorActor(UVariantObjectBinding*& OutSelectedBinding, UObject*& OutSelectedObject)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
	TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();
	if (SelectedActorNodes.Num() != 1)
	{
		return;
	}

	TSharedRef<FVariantManagerActorNode> SelectedNode = *SelectedActorNodes.CreateIterator();
	TWeakObjectPtr<UVariantObjectBinding> SelectedBinding = SelectedNode->GetObjectBinding();
	if (!SelectedBinding.IsValid())
	{
		return;
	}
	OutSelectedBinding = SelectedBinding.Get();

	// Cache all actors bound to the selected actor node's Variant
	TSet<UObject*> BoundObjects;
	UVariant* ParentVariant = SelectedBinding->GetParent();
	for (const UVariantObjectBinding* Binding : ParentVariant->GetBindings())
	{
		BoundObjects.Add(Binding->GetObject());
	}

	// See if any of the selected actors can be bound to that variant
	USelection* EditorSelection = GEditor->GetSelectedActors();
	for (FSelectionIterator SelectedObjectIter(*EditorSelection); SelectedObjectIter; ++SelectedObjectIter)
	{
		UObject* EditorSelectedObject = *SelectedObjectIter;
		if (EditorSelectedObject && !BoundObjects.Contains(EditorSelectedObject))
		{
			OutSelectedObject = EditorSelectedObject;
			return;
		}
	}
}

void SVariantManager::RebindToSelectedActor()
{
	UVariantObjectBinding* SelectedBinding = nullptr;
	UObject* SelectedObject = nullptr;
	GetSelectedBindingAndEditorActor(SelectedBinding, SelectedObject);

	AActor* SelectedActor = Cast<AActor>(SelectedObject);

	if (SelectedBinding && SelectedActor)
	{
		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("RebindToSelectedTransaction", "Rebind variant to selected actor '{0}'"),
			FText::FromString(SelectedActor->GetActorLabel())
		));

		SelectedBinding->SetObject(SelectedObject);
	}
}

bool SVariantManager::CanRebindToSelectedActor()
{
	UVariantObjectBinding* SelectedBinding = nullptr;
	UObject* SelectedActor = nullptr;
	GetSelectedBindingAndEditorActor(SelectedBinding, SelectedActor);

	if (SelectedBinding && SelectedActor)
	{
		return true;
	}

	return false;
}

void SVariantManager::RemoveActorBindings()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
		const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();

		TArray<UVariantObjectBinding*> SelectedBindings;
		for (const TSharedRef<FVariantManagerActorNode>& ActorNode : SelectedActorNodes)
		{
			UVariantObjectBinding* Binding = ActorNode->GetObjectBinding().Get();
			if (Binding)
			{
				SelectedBindings.Add(Binding);
			}
		}

		int32 NumBindings = SelectedBindings.Num();

		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("ActorNodeRemoveTransaction", "Remove {0} actor {0}|plural(one=binding,other=bindings)"),
			NumBindings
		));

		PinnedVariantManager->RemoveObjectBindingsFromParent(SelectedBindings);
		PinnedVariantManager->GetVariantManagerWidget()->RefreshActorList();
	}
}

bool SVariantManager::CanRemoveActorBindings()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
		const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = Selection.GetSelectedActorNodes();

		return SelectedActorNodes.Num() > 0;
	}

	return false;
}

void SVariantManager::ApplyProperty()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return;
	}

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	if (PropValues.Num() < 1 || !PropValues[0].IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("ApplyPropertyTransaction", "Apply recorded data for property '{0}'"),
		FText::FromString(PropValues[0].Get()->GetLeafDisplayString())));

	for (const TWeakObjectPtr<UPropertyValue>& WeakPropValue : PropValues)
	{
		if (!WeakPropValue.IsValid())
		{
			continue;
		}
		PinnedVariantManager->ApplyProperty(WeakPropValue.Get());
	}

	// Trick to force the viewport gizmos to also update, even though our selection
	// will remain the same
	GEditor->NoteSelectionChange();
}

void SVariantManager::RecordProperty()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return;
	}

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	if (PropValues.Num() < 1 || !PropValues[0].IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("ApplyPropertyTransaction", "Apply recorded data for property '{0}'"),
		FText::FromString(PropValues[0].Get()->GetLeafDisplayString())));

	for (const TWeakObjectPtr<UPropertyValue>& WeakPropValue : PropValues)
	{
		if (!WeakPropValue.IsValid())
		{
			continue;
		}
		PinnedVariantManager->RecordProperty(WeakPropValue.Get());
	}

	RefreshPropertyList();
}

void SVariantManager::RemoveCapture()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return;
	}

	TArray<UPropertyValue*> PropValuesToRemove;

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	for (const TWeakObjectPtr<UPropertyValue>& WeakPropValue : PropValues)
	{
		if (!WeakPropValue.IsValid())
		{
			continue;
		}
		PropValuesToRemove.Add(WeakPropValue.Get());
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("RemoveCaptureTransaction", "Remove {0} property {0}|plural(one=capture,other=captures)"),
		PropValuesToRemove.Num()));

	PinnedVariantManager->RemovePropertyCapturesFromParent(PropValuesToRemove);

	RefreshPropertyList();
}

void SVariantManager::CallDirectorFunction()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("TriggerEventTransaction", "Trigger a captured event"));

	for (const TSharedPtr<FVariantManagerPropertyNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Function)
		{
			TSharedPtr<FVariantManagerFunctionPropertyNode> FunctionNode = StaticCastSharedPtr<FVariantManagerFunctionPropertyNode>(Node);
			if (FunctionNode.IsValid())
			{
				FName FunctionName = FunctionNode->GetFunctionCaller().FunctionName;
				UVariantObjectBinding* FunctionTarget = FunctionNode->GetObjectBinding().Get();
				PinnedVariantManager->CallDirectorFunction(FunctionName, FunctionTarget);
			}
		}
	}
}

void SVariantManager::RemoveDirectorFunctionCaller()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);

	TMap<UVariantObjectBinding*, TArray<FFunctionCaller*>> FunctionCallers;
	int32 NumCallersWellRemove = 0;

	for (const TSharedPtr<FVariantManagerPropertyNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Function)
		{
			TSharedPtr<FVariantManagerFunctionPropertyNode> FunctionNode = StaticCastSharedPtr<FVariantManagerFunctionPropertyNode>(Node);
			if (FunctionNode.IsValid())
			{
				TArray<FFunctionCaller*>& Callers = FunctionCallers.FindOrAdd(FunctionNode->GetObjectBinding().Get());
				Callers.Add(&FunctionNode->GetFunctionCaller());

				NumCallersWellRemove++;
			}
		}
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("RemoveCallersTransaction", "Remove {0} function {0}|plural(one=caller,other=callers)"),
		NumCallersWellRemove));

	for (auto& Pair : FunctionCallers)
	{
		UVariantObjectBinding* Binding = Pair.Key;
		const TArray<FFunctionCaller*>& Callers = Pair.Value;

		PinnedVariantManager->RemoveFunctionCallers(Callers, Binding);
	}

	RefreshPropertyList();
}

bool SVariantManager::CanApplyProperty()
{
	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return false;
	}

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	if (PropValues.Num() < 1 || !PropValues[0].IsValid())
	{
		return false;
	}

	return true;
}

bool SVariantManager::CanRecordProperty()
{
	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return false;
	}

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	if (PropValues.Num() < 1 || !PropValues[0].IsValid())
	{
		return false;
	}

	return true;
}

bool SVariantManager::CanRemoveCapture()
{
	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return false;
	}

	const TArray<TWeakObjectPtr<UPropertyValue>>& PropValues = SelectedNodes[0]->GetPropertyValues();
	if (PropValues.Num() < 1 || !PropValues[0].IsValid())
	{
		return false;
	}

	return true;
}

bool SVariantManager::CanCallDirectorFunction()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<FVariantManagerPropertyNode>> SelectedNodes;
	int32 NumNodes = CapturedPropertyListView->GetSelectedItems(SelectedNodes);
	if (NumNodes != 1)
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("TriggerEventTransaction", "Trigger a captured event"));

	for (const TSharedPtr<FVariantManagerPropertyNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Function)
		{
			TSharedPtr<FVariantManagerFunctionPropertyNode> FunctionNode = StaticCastSharedPtr<FVariantManagerFunctionPropertyNode>(Node);
			if (FunctionNode.IsValid())
			{
				FFunctionCaller& Caller = FunctionNode->GetFunctionCaller();
				return Caller.IsValidFunction(Caller.GetFunctionEntry());
			}
		}
	}

	return false;
}

bool SVariantManager::CanRemoveDirectorFunctionCaller()
{
	return true;
}

void SVariantManager::SwitchOnVariant(UVariant* Variant)
{
	if (Variant == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("SwitchOnVariantTransaction", "Switch on variant '{0}'"),
		Variant->GetDisplayText()));

	bool bSomeFailedToResolve = false;
	for (const UVariantObjectBinding* Binding : Variant->GetBindings())
	{
		if (!Binding->GetObject())
		{
			bSomeFailedToResolve = true;
			break;
		}
	}
	if (bSomeFailedToResolve)
	{
		FNotificationInfo Error(FText::Format(LOCTEXT("UnresolvedActorsOnSwitchOnNotification", "Switched-on Variant '{0}' contains unresolved actor bindings!"),
			Variant->GetDisplayText()));
		Error.ExpireDuration = 5.0f;
		Error.bFireAndForget = true;
		Error.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
		FSlateNotificationManager::Get().AddNotification(Error);
	}

	Variant->SwitchOn();

	// Trick to force the viewport gizmos to also update, even though our selection
	// may remain the same
	GEditor->NoteSelectionChange();

	// Force a redraw to save some confusion in case the user has Realtime: Off.
	// Do this on next frame because there is some minor issue where if visibility changes are triggered
	// on the same frame that is meant to be invalidated, sometimes the primitive's occlusion history doesn't
	// refresh properly and we get some incorrectly hidden/visible objects (UE-100896)
	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([](float Time)
		{
			const bool bInvalidateHitProxies = false;
			GEditor->RedrawLevelEditingViewports( bInvalidateHitProxies );
			return false;
		})
	);
}

void SVariantManager::SortDisplayNodes(TArray<TSharedRef<FVariantManagerDisplayNode>>& DisplayNodes)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	struct NodeAndDisplayIndex
	{
		TSharedRef<FVariantManagerDisplayNode> Node;
		int32 DisplayIndex;
	};

	TArray<NodeAndDisplayIndex> SortedNodes;
	SortedNodes.Reserve(DisplayNodes.Num());

	for (TSharedRef<FVariantManagerDisplayNode>& DisplayNode : DisplayNodes)
	{
		int32 Index = INDEX_NONE;

		switch (DisplayNode->GetType())
		{
		case EVariantManagerNodeType::Actor:
			Index = DisplayedActors.Find(DisplayNode);
			break;
		case EVariantManagerNodeType::Variant:
		case EVariantManagerNodeType::VariantSet:
			Index = NodeTreeView->GetDisplayIndexOfNode(DisplayNode);
			break;
		default:
			break;
		}

		SortedNodes.Add({DisplayNode, Index});
	}

	SortedNodes.Sort([](const NodeAndDisplayIndex& A, const NodeAndDisplayIndex& B)
	{
		return A.DisplayIndex < B.DisplayIndex;
	});

	DisplayNodes.Empty(DisplayNodes.Num());
	for (const NodeAndDisplayIndex& SortedNode : SortedNodes)
	{
		DisplayNodes.Add(SortedNode.Node);
	}
}

// Utility that scans the passed in display nodes and returns all the contained variants and variant sets
void GetVariantsAndVariantSetsFromNodes(const TArray<TSharedRef<FVariantManagerDisplayNode>>& InNodes, TSet<UVariant*>& OutVariants, TSet<UVariantSet*>& OutVariantSets)
{
	for (const TSharedRef<FVariantManagerDisplayNode>& DisplayNode : InNodes)
	{
		if (DisplayNode->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> DisplayNodeAsVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(DisplayNode);
			if (DisplayNodeAsVarNode.IsValid())
			{
				OutVariants.Add(&DisplayNodeAsVarNode->GetVariant());
				continue;
			}
		}
		else if (DisplayNode->GetType() == EVariantManagerNodeType::VariantSet)
		{
			TSharedPtr<FVariantManagerVariantSetNode> DisplayNodeAsVarSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(DisplayNode);
			if (DisplayNodeAsVarSetNode.IsValid())
			{
				OutVariantSets.Add(&DisplayNodeAsVarSetNode->GetVariantSet());
				continue;
			}
		}
	}
}

void SVariantManager::OnActorNodeSelectionChanged()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		// Prevent OnEditorSelectionChanged from firing when we update this selection
		TGuardValue<bool> Guard(bRespondToEditorSelectionEvents, false);

		const TSet<TSharedRef<FVariantManagerActorNode>>& SelectedActorNodes = PinnedVariantManager->GetSelection().GetSelectedActorNodes();

		GEditor->SelectNone(true, true);

		for (const TSharedRef<FVariantManagerActorNode>& ActorNode : SelectedActorNodes)
		{
			TWeakObjectPtr<UVariantObjectBinding> Binding = ActorNode->GetObjectBinding();
			if (Binding.IsValid())
			{
				if (AActor* SelectedActor = Cast<AActor>(Binding->GetObject()))
				{
					GEditor->SelectActor(SelectedActor, true, true);
				}
			}
		}
	}

	RefreshPropertyList();
}

void SVariantManager::OnVariantDependenciesUpdated(UVariant* ParentVariant)
{
	if (!ParentVariant)
	{
		return;
	}

	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	TArray<UVariant*> SelectedVariants;
	TArray<UVariantSet*> SelectedVariantSets;
	PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
	if (!SelectedVariants.Contains(ParentVariant))
	{
		return;
	}

	RefreshDependencyLists();
}

void SVariantManager::RefreshVariantTree()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}
	FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
	TSet<FString>& SelectedNodePaths = Selection.GetSelectedNodePaths();

	// Store previous selection
	for (const TSharedRef<FVariantManagerDisplayNode>& DisplayNode : Selection.GetSelectedOutlinerNodes())
	{
		if (DisplayNode->GetType() == EVariantManagerNodeType::VariantSet)
		{
			TSharedPtr<FVariantManagerVariantSetNode> DisplayNodeAsVarSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(DisplayNode);
			if (DisplayNodeAsVarSetNode.IsValid())
			{
				SelectedNodePaths.Add(DisplayNodeAsVarSetNode->GetVariantSet().GetPathName());
			}
		}
		else if (DisplayNode->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> DisplayNodeAsVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(DisplayNode);
			if (DisplayNodeAsVarNode.IsValid())
			{
				SelectedNodePaths.Add(DisplayNodeAsVarNode->GetVariant().GetPathName());
			}
		}
	}

	// Store selected UVariant and UVariantSets so that we can re-select them after the rebuild if we can
	TSet<UVariant*> OldSelectedVariants;
	TSet<UVariantSet*> OldSelectedVariantSets;
	GetVariantsAndVariantSetsFromNodes(Selection.GetSelectedOutlinerNodes().Array(), OldSelectedVariants, OldSelectedVariantSets);

	Selection.SuspendBroadcast();
	Selection.EmptySelectedOutlinerNodes();

	PinnedVariantManager->GetNodeTree()->Update();

	// Restore the selection state.
	for (const TSharedRef<FVariantManagerDisplayNode>& DisplayNode : PinnedVariantManager->GetNodeTree()->GetRootNodes())
	{
		if (DisplayNode->GetType() == EVariantManagerNodeType::VariantSet)
		{
			TSharedPtr<FVariantManagerVariantSetNode> DisplayNodeAsVarSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(DisplayNode);
			if (DisplayNodeAsVarSetNode.IsValid())
			{
				if (SelectedNodePaths.Contains(DisplayNodeAsVarSetNode->GetVariantSet().GetPathName()))
				{
					Selection.AddToSelection(DisplayNode);
				}

				for (const TSharedRef<FVariantManagerDisplayNode>& ChildDisplayNode : DisplayNodeAsVarSetNode->GetChildNodes())
				{
					if (ChildDisplayNode->GetType() == EVariantManagerNodeType::Variant)
					{
						TSharedPtr<FVariantManagerVariantNode> ChildDisplayNodeAsVarNode = StaticCastSharedRef<FVariantManagerVariantNode>(ChildDisplayNode);
						if (ChildDisplayNodeAsVarNode.IsValid())
						{
							if (SelectedNodePaths.Contains(ChildDisplayNodeAsVarNode->GetVariant().GetPathName()))
							{
								Selection.AddToSelection(ChildDisplayNode);
							}
						}
					}
				}
			}
		}
	}

	// Do this now or else we might have dangling paths that will be randomly selected when we replace a node
	SelectedNodePaths.Empty();

	NodeTreeView->UpdateTreeViewFromSelection();
	NodeTreeView->Refresh();
	Selection.ResumeBroadcast();
}

void SVariantManager::RefreshActorList()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}
	FVariantManagerSelection& Selection = PinnedVariantManager->GetSelection();
	TSet<FString>& SelectedNodePaths = Selection.GetSelectedNodePaths();

	// Store previous actor selection
	for (const TSharedRef<FVariantManagerActorNode>& SelectedActorNode : Selection.GetSelectedActorNodes())
	{
		SelectedNodePaths.Add(SelectedActorNode->GetObjectBinding()->GetPathName());
	}

	// Rebuild list of FVariantManagerActorNode
	{
		CachedDisplayedActorPaths.Empty();

		// Get all unique variants in order (in case we selected a variant and its variant set)
		TArray<UVariant*> SelectedVariants;
		for (const TSharedRef<FVariantManagerDisplayNode>& Node : Selection.GetSelectedOutlinerNodes())
		{
			if (Node->GetType() == EVariantManagerNodeType::VariantSet)
			{
				TSharedRef<FVariantManagerVariantSetNode> NodeAsVarSet = StaticCastSharedRef<FVariantManagerVariantSetNode>(Node);
				const TArray<UVariant*>& Variants = NodeAsVarSet->GetVariantSet().GetVariants();

				for (UVariant* Variant : Variants)
				{
					SelectedVariants.AddUnique(Variant);
				}
			}
			else if (Node->GetType() == EVariantManagerNodeType::Variant)
			{
				SelectedVariants.AddUnique(&StaticCastSharedRef<FVariantManagerVariantNode>(Node)->GetVariant());
			}
		}

		// Get all bindings to use, in order (but allowing repeats because we might have selected two variants
		// with bindings to the same actor)
		TArray<UVariantObjectBinding*> TargetBindings;
		for (UVariant* Variant : SelectedVariants)
		{
			const TArray<UVariantObjectBinding*>& Bindings = Variant->GetBindings();
			if (Bindings.Num() > 0)
			{
				TargetBindings.Append(Bindings);
			}
		}

		DisplayedActors.Empty();
		for (UVariantObjectBinding* Binding : TargetBindings)
		{
			if (Binding == nullptr)
			{
				DisplayedActors.Add(MakeShareable(new FVariantManagerDisplayNode(nullptr, nullptr)));
			}
			else
			{
				DisplayedActors.Add(MakeShareable(new FVariantManagerActorNode(Binding, nullptr, VariantManagerPtr)));
				CachedDisplayedActorPaths.Add(Binding->GetObjectPath());
			}
		}
	}

	// Restore actor selection
	Selection.SuspendBroadcast();
	Selection.EmptySelectedActorNodes();
	CachedSelectedActorPaths.Empty();

	for (const TSharedRef<FVariantManagerDisplayNode>& DisplayedNode : DisplayedActors)
	{
		if (DisplayedNode->GetType() == EVariantManagerNodeType::Actor)
		{
			const TSharedRef<FVariantManagerActorNode>& DisplayedActor = StaticCastSharedRef<FVariantManagerActorNode>(DisplayedNode);

			TWeakObjectPtr<UVariantObjectBinding> Binding = DisplayedActor->GetObjectBinding();
			if (Binding.IsValid() && SelectedNodePaths.Contains(Binding->GetPathName()))
			{
				Selection.AddActorNodeToSelection(DisplayedActor);
				CachedSelectedActorPaths.Add(Binding->GetObjectPath());
			}
		}
	}

	SelectedNodePaths.Empty();

	// Select the FVariantManagerSelection items in the SListView
	ActorListView->UpdateListViewFromSelection();
	ActorListView->RebuildList();
	Selection.ResumeBroadcast();

	// We might be still selecting a binding to the same actor, but we need to update
	// the captured properties, because we might select a different variant now, so the captured
	// properties could be different
	RefreshPropertyList();
	RefreshDependencyLists();
}

namespace SVariantManagerUtils
{
	enum class ERowType : uint8
	{
		None = 0,
		PropertyValue = 1,
		FunctionCaller = 2,
	};

	struct FPropertyPanelRow
	{
		FPropertyPanelRow(ERowType InRowType, TArray<UPropertyValue*>&& InPropertyValues)
			: PropertyValues(InPropertyValues)
			, FunctionCaller(nullptr)
			, Binding(nullptr)
			, RowType(InRowType)
			, DisplayOrder(UINT32_MAX)
		{
			ensure(RowType == ERowType::PropertyValue);

			for (UPropertyValue* PropertyValue : PropertyValues)
			{
				DisplayOrder = FMath::Min(DisplayOrder, PropertyValue->GetDisplayOrder());
			}
		}

		FPropertyPanelRow(ERowType InRowType, FFunctionCaller* InFunctionCaller, UVariantObjectBinding* InBinding)
			: FunctionCaller(InFunctionCaller)
			, Binding(InBinding)
			, RowType(InRowType)
			, DisplayOrder(UINT32_MAX)
		{
			ensure(RowType == ERowType::FunctionCaller);

			DisplayOrder = FunctionCaller->GetDisplayOrder();
		}

		FName GetName() const
		{
			if (RowType == ERowType::PropertyValue && PropertyValues.Num() > 0)
			{
				return *PropertyValues[0]->GetFullDisplayString();
			}
			else if (RowType == ERowType::FunctionCaller && FunctionCaller)
			{
				return FunctionCaller->FunctionName;
			}

			return NAME_None;
		}

		TArray<UPropertyValue*> PropertyValues;

		FFunctionCaller* FunctionCaller;
		UVariantObjectBinding* Binding;

		ERowType RowType;
		uint32 DisplayOrder;
	};
}

void SVariantManager::RefreshPropertyList()
{
	// Don't update if the properties tab is collapsed
	if ( CapturedPropertyListView == nullptr )
	{
		return;
	}

	FVariantManagerSelection& Selection = VariantManagerPtr.Pin()->GetSelection();

	TArray<UPropertyValue*> Properties;

	TArray<SVariantManagerUtils::FPropertyPanelRow> Rows;

	for (const TSharedRef<FVariantManagerActorNode>& Node : Selection.GetSelectedActorNodes())
	{
		UVariantObjectBinding* Binding = Node->GetObjectBinding().Get();
		if (Binding == nullptr || Binding->GetObject() == nullptr)
		{
			continue;
		}

		Properties.Append(Binding->GetCapturedProperties());

		// Add function caller rows first as we need the bindings
		for (FFunctionCaller& Caller : Binding->GetFunctionCallers())
		{
			Rows.Emplace(SVariantManagerUtils::ERowType::FunctionCaller, &Caller, Binding);
		}
	}

	// Group properties with the same hash, to show a "multiple object" type of property editor
	TMap<uint32, TArray<UPropertyValue*>> PropsByHash;
	for (UPropertyValue* Property : Properties)
	{
		if (!Property)
		{
			continue;
		}

		uint32 Hash = Property->GetPropertyPathHash();
		PropsByHash.FindOrAdd(Hash).Add(Property);
	}

	// Add the grouped property rows
	for (TTuple<uint32, TArray<UPropertyValue*>>& HashToPropArray : PropsByHash)
	{
		Rows.Emplace(SVariantManagerUtils::ERowType::PropertyValue, MoveTemp(HashToPropArray.Value));
	}
	PropsByHash.Reset();

	Rows.Sort([](const SVariantManagerUtils::FPropertyPanelRow& Left, const SVariantManagerUtils::FPropertyPanelRow& Right)
	{
		if (Left.DisplayOrder == Right.DisplayOrder)
		{
			return Left.GetName().LexicalLess(Right.GetName());
	}

		return Left.DisplayOrder < Right.DisplayOrder;
	});

	DisplayedPropertyNodes.Empty();
	for(SVariantManagerUtils::FPropertyPanelRow& Row : Rows)
	{
		if (Row.RowType == SVariantManagerUtils::ERowType::PropertyValue)
		{
			TArray<UPropertyValue*>& Props = Row.PropertyValues;
			if (Props.Num() < 1)
			{
				continue;
			}

			// Attempts to resolve first so that we can fetch the objects below
			UPropertyValue* FirstProp = Props[0];
			FirstProp->Resolve();

			UScriptStruct* Struct = FirstProp->GetStructPropertyStruct();
			UEnum* Enum = FirstProp->GetEnumPropertyEnum();

			if (Struct)
			{
				FName StructName = Struct->GetFName();
				if (StructName == NAME_Color || StructName == NAME_LinearColor)
				{
					DisplayedPropertyNodes.Add(MakeShared<FVariantManagerColorPropertyNode>(Props, VariantManagerPtr));
				}
				else
				{
					DisplayedPropertyNodes.Add(MakeShared<FVariantManagerStructPropertyNode>(Props, VariantManagerPtr));
				}
			}
			else if (Enum)
			{
				DisplayedPropertyNodes.Add(MakeShared<FVariantManagerEnumPropertyNode>(Props, VariantManagerPtr));
			}
			else if (FirstProp->GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()) ||
					 FirstProp->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()) ||
					 FirstProp->GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
			{
				DisplayedPropertyNodes.Add(MakeShared<FVariantManagerStringPropertyNode>(Props, VariantManagerPtr));
			}
			else if (FirstProp->GetPropCategory() == EPropertyValueCategory::Option)
			{
				DisplayedPropertyNodes.Add(MakeShared<FVariantManagerOptionPropertyNode>(Props, VariantManagerPtr));
			}
			else
			{
				DisplayedPropertyNodes.Add(MakeShared<FVariantManagerPropertyNode>(Props, VariantManagerPtr));
			}
		}
		else if (Row.RowType == SVariantManagerUtils::ERowType::FunctionCaller && Row.Binding && Row.FunctionCaller)
		{
			DisplayedPropertyNodes.Add(MakeShared<FVariantManagerFunctionPropertyNode>(Row.Binding, *Row.FunctionCaller, VariantManagerPtr));
		}
	}

	CapturedPropertyListView->RequestListRefresh();
}

void SVariantManager::RefreshDependencyLists()
{
	if (!DependenciesList)
	{
		return;
	}

	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	DisplayedDependencies.Reset();

	TArray<UVariant*> SelectedVariants;
	TArray<UVariantSet*> SelectedVariantSets;
	PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
	if (SelectedVariants.Num() == 1)
	{
		UVariant* SelectedVariant = SelectedVariants[0];

		for (int32 Index = 0; Index < SelectedVariant->GetNumDependencies(); ++Index)
		{
			FVariantDependencyModelPtr Model = MakeShared<FVariantDependencyModel>(
				SelectedVariant, 
				&SelectedVariant->GetDependency(Index), 
				false,
				false);

			DisplayedDependencies.Add(Model);
		}

		const bool bOnlyEnabledDependencies = false;
		TArray<UVariant*> Dependents = SelectedVariant->GetDependents(PinnedVariantManager->GetCurrentLevelVariantSets(), bOnlyEnabledDependencies);
		
		if (Dependents.Num() > 0)
		{
			// Add divider between the dependencies list and dependents list
			DisplayedDependencies.Add(MakeShared<FVariantDependencyModel>(nullptr, nullptr, false, true));
		}

		for (UVariant* Dependent : Dependents)
		{
			FVariantDependencyModelPtr Model = MakeShared<FVariantDependencyModel>(
				Dependent,
				nullptr,
				true,
				false);

			DisplayedDependencies.Add(Model);
		}
	}

	DependenciesList->RequestListRefresh();
}

void SVariantManager::UpdatePropertyDefaults()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		if (ULevelVariantSets* LVS = PinnedVariantManager->GetCurrentLevelVariantSets())
		{
			for (UVariantSet* VarSet : LVS->GetVariantSets())
			{
				for (UVariant* Var : VarSet->GetVariants())
				{
					for (UVariantObjectBinding* Binding : Var->GetBindings())
					{
						for (UPropertyValue* Prop : Binding->GetCapturedProperties())
						{
							Prop->ClearDefaultValue();
						}
					}
				}
			}
		}
	}
}

void SVariantManager::OnBlueprintCompiled()
{
	RefreshPropertyList();

	// We might have changed the default value for a blueprint component or actor
	UpdatePropertyDefaults();
}

void SVariantManager::OnMapChanged(UWorld* World, EMapChangeType MapChangeType)
{
	CachedAllActorPaths.Empty();
	RefreshActorList();
}

void SVariantManager::OnOutlinerSearchChanged(const FText& Filter)
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();
	if (VariantManager.IsValid())
	{
		const FString& FilterString = Filter.ToString();

		VariantManager->GetNodeTree()->FilterNodes(FilterString);
		NodeTreeView->Refresh();
	}
}

void SVariantManager::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
}

void SVariantManager::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
}

FReply SVariantManager::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	//bool bIsDragSupported = false;

	//TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	//if (Operation.IsValid() && (
	//	Operation->IsOfType<FAssetDragDropOp>() ||
	//	Operation->IsOfType<FClassDragDropOp>() ||
	//	Operation->IsOfType<FActorDragDropGraphEdOp>() ) )
	//{
	//	bIsDragSupported = true;
	//}

	//return bIsDragSupported ? FReply::Handled() : FReply::Unhandled();
	return FReply::Unhandled();
}

FReply SVariantManager::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	//bool bWasDropHandled = false;

	//TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	//if (Operation.IsValid() )
	//{
	//	if ( Operation->IsOfType<FAssetDragDropOp>() )
	//	{
	//		const auto& DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	//		//OnAssetsDropped( *DragDropOp );
	//		bWasDropHandled = true;
	//	}
	//	else if( Operation->IsOfType<FClassDragDropOp>() )
	//	{
	//		const auto& DragDropOp = StaticCastSharedPtr<FClassDragDropOp>( Operation );

	//		//OnClassesDropped( *DragDropOp );
	//		bWasDropHandled = true;
	//	}
	//	else if( Operation->IsOfType<FActorDragDropGraphEdOp>() )
	//	{
	//		const auto& DragDropOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>( Operation );

	//		//OnActorsDropped( *DragDropOp );
	//		bWasDropHandled = true;
	//	}
	//}

	//return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
	return FReply::Unhandled();
}

FReply SVariantManager::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// A toolkit tab is active, so direct all command processing to it
	if(VariantTreeCommandBindings->ProcessCommandBindings( InKeyEvent ))
	{
		return FReply::Handled();
	}

	const FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Enter)
	{
		SwitchOnSelectedVariant();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SVariantManager::OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent )
{
	// if (NewWidgetPath.ContainsWidget(AsShared()))
	// {
	// 	OnReceivedFocus.ExecuteIfBound();
	// }
}

FReply SVariantManager::OnAddVariantSetClicked()
{
	FScopedTransaction Transaction(LOCTEXT("AddVariantSetTransaction", "Create a new variant set"));

	CreateNewVariantSet();
	return FReply::Handled();
}

FReply SVariantManager::OnSummonAddActorMenu()
{
	TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
	if (!VarMan.IsValid())
	{
		return FReply::Unhandled();
	}

	TArray<UVariant*> SelectedVariants;
	TArray<UVariantSet*> SelectedVariantSets;
	VarMan->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
	if (SelectedVariants.Num() + SelectedVariantSets.Num() == 0)
	{
		return FReply::Unhandled();
	}

	for (UVariantSet* VarSet : SelectedVariantSets)
	{
		if (!VarSet)
		{
			continue;
		}

		SelectedVariants.Append(VarSet->GetVariants());
	}

	// Find the set of actors that is already bound to all selected variants: We can't add bindings to those
	TArray<TSet<const AActor*>> BoundActorSets;
	BoundActorSets.SetNum(SelectedVariants.Num());
	for (int32 VariantIndex = 0; VariantIndex < SelectedVariants.Num(); ++VariantIndex)
	{
		UVariant* SelectedVariant = SelectedVariants[VariantIndex];
		if (!SelectedVariant)
		{
			continue;
		}

		const TArray<UVariantObjectBinding*>& Bindings = SelectedVariant->GetBindings();

		TSet<const AActor*>& BoundActors = BoundActorSets[VariantIndex];
		BoundActors.Reserve(Bindings.Num());
		for (UVariantObjectBinding* Binding : Bindings)
		{
			if (!Binding)
			{
				continue;
			}

			if (AActor* BoundActor = Cast<AActor>(Binding->GetObject()))
			{
				BoundActors.Add(BoundActor);
			}
		}
	}
	TSet<const AActor*> CommonActorSet;
	if (BoundActorSets.Num() > 0)
	{
		CommonActorSet = BoundActorSets[0];
		for (int32 Index = 1; Index < BoundActorSets.Num(); ++Index)
		{
			CommonActorSet = CommonActorSet.Intersect(BoundActorSets[Index]);
		}
	}

	auto IsActorValidForAssignment = [CommonActorSet](const AActor* InActor)
	{
		return !CommonActorSet.Contains(InActor);
	};

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = true;
	InitOptions.bShowSearchBox = true;
	InitOptions.bShowCreateNewFolder = false;
	InitOptions.bFocusSearchBoxWhenOpened = true;
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));
	InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda( IsActorValidForAssignment ));

	// Create mini scene outliner menu
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef< SWidget > MiniSceneOutliner =
		SNew( SBox )
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([SelectedVariants, VarMan, this](AActor* Actor)
				{
					if (!Actor)
					{
						return;
					}

					FSlateApplication::Get().DismissAllMenus();

					FScopedTransaction Transaction(FText::Format(
						LOCTEXT("AddActorPlusTransaction", "Add actor '{0}' to selected variants"),
						FText::FromString(Actor->GetActorLabel())));

					VarMan->CreateObjectBindings({Actor}, SelectedVariants);
					RefreshActorList();
				})
			)
		];

	// I'm not sure why we need a menu builder here as opposed to just passing the MiniSceneOutliner
	// directly to PushMenu. If we do that though we get a bunch of drawing glitches
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection(TEXT("ChooseActorSection"), LOCTEXT("ChooseActor", "Choose Actor:"));
	MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);

	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();
	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		MouseCursorLocation,
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
	);

	return FReply::Handled();
}

FReply SVariantManager::OnAddDependencyClicked()
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return FReply::Handled();
	}

	TArray<UVariant*> Variants;
	TArray<UVariantSet*> VariantSets;
	PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Variants, VariantSets);
	if (Variants.Num() == 1 && Variants[0] != nullptr)
	{
		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("AddDependencyTransaction", "Add a dependency to variant '{0}'"),
			Variants[0]->GetDisplayText()
		));

		UVariant* SelectedVariant = Variants[0];

		FVariantDependency Dependency;
		SelectedVariant->AddDependency(Dependency);
	}

	return FReply::Handled();
}

// Tries capturing and recording new data for the property at PropertyPath for TargetActor, into whatever Variants we have selected.
// Will return true if it created or updated a UPropertyValue
bool AutoCaptureProperty(FVariantManager* VarMan, AActor* TargetActor, const FString& PropertyPath, const FProperty* Property)
{
	// Transient actors are generated temporarily while dragging actors into the level. Once the
	// mouse is released, another non-transient actor is instantiated
	if (!VarMan || !TargetActor || TargetActor->HasAnyFlags(RF_Transient) || PropertyPath.IsEmpty())
	{
		return false;
	}

	// Get selected variants
	TArray<UVariant*> SelectedVariants;
	TArray<UVariantSet*> SelectedVariantSets;
	VarMan->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
	if (SelectedVariants.Num() < 1)
	{
		return false;
	}

	// Create/get bindings
	TArray<AActor*> TargetActorArr{TargetActor};
	TArray<UVariantObjectBinding*> Bindings = VarMan->CreateObjectBindings(TargetActorArr, SelectedVariants);
	if (Bindings.Num() < 1)
	{
		return false;
	}

	// Create property captures
	TArray<TSharedPtr<FCapturableProperty>> OutProps;
	VarMan->GetCapturableProperties(TargetActorArr, OutProps, PropertyPath, true);
	if (OutProps.Num() == 0)
	{
		return false;
	}

	TArray<UPropertyValue*> CreatedProps = VarMan->CreatePropertyCaptures(OutProps, {Bindings}, true);

	// UPropertyValue always contains the Inner for array properties, but the event that
	// calls this function only provides the outer
	FProperty* FilterProperty = const_cast<FProperty*>(Property);
	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
	{
		FilterProperty = ArrayProp->Inner;
	}

	// Exception for switch actor, as FilterProperty will be nullptr because the "Selected Option" property doesn't really exist
	bool bIsSelectedOption = Cast<ASwitchActor>(TargetActor) && PropertyPath == SWITCH_ACTOR_SELECTED_OPTION_NAME;

	// Update property captures
	for (UVariantObjectBinding* Binding : Bindings)
	{
		for (UPropertyValue* PropertyValue : Binding->GetCapturedProperties())
		{
			if ((FilterProperty && PropertyValue->ContainsProperty(FilterProperty)) ||
				(Cast<UPropertyValueOption>(PropertyValue) && bIsSelectedOption))
			{
				PropertyValue->RecordDataFromResolvedObject();
			}
		}
	}

	return true;
}

namespace SVariantManagerUtils
{
	// Returns the paths of all the actors bound to variants of this LVS
	TSet<FString> GetAllActorPaths(const ULevelVariantSets* LVS)
	{
		if (LVS == nullptr)
		{
			return {};
		}

		TSet<FString> Result;

		for (const UVariantSet* VarSet : LVS->GetVariantSets())
		{
			if (VarSet == nullptr)
			{
				continue;
			}

			for (const UVariant* Var : VarSet->GetVariants())
			{
				if (Var == nullptr)
				{
					continue;
				}

				for (const UVariantObjectBinding* Binding : Var->GetBindings())
				{
					if (Binding == nullptr)
					{
						continue;
					}

					// Need to do this instead of just asking the binding for its
					// path because we need the paths fixed up for PIE, if that is the case
					if (UObject* Actor = Binding->GetObject())
					{
						Result.Add(Actor->GetPathName());
					}
				}
			}
		}

		return Result;
	}

	enum class EObjectType : uint8
	{
		None = 0,
		PropertyValue = 1,
		VariantObjectBinding = 2,
		Variant = 4,
		VariantSet = 8,
		LevelVariantSets = 16,
	};
	ENUM_CLASS_FLAGS(EObjectType);

	EObjectType GetObjectType(UObject* Object)
	{
		if (Object == nullptr)
		{
			return EObjectType::None;
		}

		UClass* ObjectClass = Object->GetClass();
		if (ObjectClass->IsChildOf(UPropertyValue::StaticClass()))
		{
			return EObjectType::PropertyValue;
		}
		else if (ObjectClass->IsChildOf(UVariantObjectBinding::StaticClass()))
		{
			return EObjectType::VariantObjectBinding;
		}
		else if (ObjectClass->IsChildOf(UVariant::StaticClass()))
		{
			return EObjectType::Variant;
		}
		else if (ObjectClass->IsChildOf(UVariantSet::StaticClass()))
		{
			return EObjectType::VariantSet;
		}
		else if (ObjectClass->IsChildOf(ULevelVariantSets::StaticClass()))
		{
			return EObjectType::LevelVariantSets;
		}

		return EObjectType::None;
	}
}

void SVariantManager::OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& Event)
{
	// We fully redraw the variant manager when undoing/redoing, so we can just worry about finalized here
	if (Event.GetEventType() != ETransactionObjectEventType::Finalized)
	{
		return;
	}

	if (Object == nullptr)
	{
		return;
	}

	SVariantManagerUtils::EObjectType ObjectType = SVariantManagerUtils::GetObjectType(Object);

	// Variants may have changed 'active' state
	if(EnumHasAnyFlags(ObjectType, SVariantManagerUtils::EObjectType::PropertyValue |
								   SVariantManagerUtils::EObjectType::VariantObjectBinding |
								   SVariantManagerUtils::EObjectType::Variant))
	{
		RefreshVariantTree();
	}

	// Set of all bound actors may have changed
	if (EnumHasAnyFlags(ObjectType, SVariantManagerUtils::EObjectType::VariantObjectBinding |
									SVariantManagerUtils::EObjectType::Variant |
									SVariantManagerUtils::EObjectType::VariantSet |
									SVariantManagerUtils::EObjectType::LevelVariantSets))
	{
		CachedAllActorPaths.Empty();
	}

	AActor* TargetActor = Cast<AActor>(Object);
	if (TargetActor == nullptr)
	{
		if (UActorComponent* ObjectAsActorComponent = Cast<UActorComponent>(Object))
		{
			TargetActor = ObjectAsActorComponent->GetOwner();
		}
	}

	// Actor or an actor's component has transacted
	if (TargetActor != nullptr)
	{
		FString ActorPath = TargetActor->GetPathName();

		// When we switch a SwitchActor, only the child actors will transact, so we
		// have to manually check if this transaction was a switch actor switch
		FString ParentActorPath;
		if (ASwitchActor* SwitchActorParent = Cast<ASwitchActor>(TargetActor->GetAttachParentActor()))
		{
			if (Object->IsA(USceneComponent::StaticClass()) && Event.GetChangedProperties().Contains(USceneComponent::GetVisiblePropertyName()))
			{
				ParentActorPath = SwitchActorParent->GetPathName();
				bool bSwitchWasCapturedAlready = CachedDisplayedActorPaths.Contains(ParentActorPath);

				// Annoyingly we have to handle switch actor auto-capture in here, as it doesn't have
				// any 'property' to trigger OnObjectPropertyChanged
				if (bAutoCaptureProperties)
				{
					TSharedPtr<FVariantManager> PinnedVarMan = VariantManagerPtr.Pin();
					bool bDidSomething = AutoCaptureProperty(PinnedVarMan.Get(), SwitchActorParent, SWITCH_ACTOR_SELECTED_OPTION_NAME, nullptr);

					if (bDidSomething && !bSwitchWasCapturedAlready)
					{
						RefreshActorList();
					}
				}
			}
		}

		// Recorded values may be out of date, so we would need to show the "Record" button (aka dirty
		// property indicator)
		if (CachedSelectedActorPaths.Contains(ActorPath) || CachedSelectedActorPaths.Contains(ParentActorPath))
		{
			RefreshPropertyList();
		}

		// Make sure this cache is built
		if (CachedAllActorPaths.Num() == 0)
		{
			TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
			if (VarMan.IsValid())
			{
				TSet<FString> DiscoveredActorPaths = SVariantManagerUtils::GetAllActorPaths(VarMan->GetCurrentLevelVariantSets());
				CachedAllActorPaths = MoveTemp(DiscoveredActorPaths);
			}
		}

		// If the actor transacted, properties may not be current and so variants may not be active anymore
		if (CachedAllActorPaths.Contains(ActorPath) || CachedAllActorPaths.Contains(ParentActorPath))
		{
			RefreshVariantTree();
		}
	}
}

namespace SVariantManagerUtils
{
	struct FReconstructionResult
	{
		FString PathActorToLeafComponent;
		FString PathTailProperty;
		AActor* TargetActor = nullptr;
		USceneComponent* LeafComponent = nullptr;
	};

	// This needs to be a separate function because OnPreObjectPropertyChanged is required to parse generic properties and
	// disambiguate property paths (as it provides the full path), but it does not fire for some changes, like Materials.
	// OnObjectPropertyChanged seems to fire for everything, but it doesn't provide the full path. This means that we sometimes need
	// to reconstruct the path from OnObjectPropertyChanged, and sometimes from OnPreObjectPropertyChanged
	FReconstructionResult ReconstructPropertyPath(UObject* SomeObject, FProperty* ParentProperty, FProperty* ChildProperty)
	{
		FReconstructionResult Result;
		if (!SomeObject || !ParentProperty)
		{
			return Result;
		}

		FString ComponentPath;

		AActor* TargetActor = nullptr;
		USceneComponent* LeafComponent = Cast<USceneComponent>(SomeObject);
		if (LeafComponent)
		{
			TargetActor = LeafComponent->GetOwner();
		}
		else if (UActorComponent* ObjAsActorComp = Cast<UActorComponent>(SomeObject))
		{
			TargetActor = ObjAsActorComp->GetOwner();

			// Actor components can't be attached to scene components, so we know we're
			// attached directly to the actor and there are no other components in the chain
			ComponentPath = ObjAsActorComp->GetName() + PATH_DELIMITER;
		}
		else
		{
			TargetActor = Cast<AActor>(SomeObject);
		}
		if (!TargetActor)
		{
			return Result;
		}

		// Some AActor types have direct UPROPERTY pointer "shortcuts" to components that are on their own
		// component hierarchies (e.g. ALight, ACameraActor, etc). On those cases the property pointers may be
		// those shortcut components, and our picture of the component hierarchy could be incorrect.
		// What we do know is that we can always resolve the parent property on the Object itself to retrieve a
		// valid Component, so here we just travel it upwards to complete the path avoiding any shortcuts
		if (TargetActor == SomeObject)
		{
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ParentProperty))
			{
				void* ObjectContainer = ObjectProperty->ContainerPtrToValuePtr<void>(TargetActor);
				UObject* TargetObject = ObjectProperty->GetObjectPropertyValue(ObjectContainer);

				if (USceneComponent* StartingSceneComp = Cast<USceneComponent>(TargetObject))
				{
					if (StartingSceneComp->GetOwner() == TargetActor)
					{
						LeafComponent = StartingSceneComp;
					}
				}
				else if (UActorComponent* StartingActorComp = Cast<UActorComponent>(TargetObject))
				{
					if (StartingActorComp->GetOwner() == TargetActor)
					{
						// Actor components can't be attached to scene components, so we know we're
						// attached directly to the actor and there are no other components in the chain
						ComponentPath = StartingActorComp->GetName() + PATH_DELIMITER;
						LeafComponent = nullptr;
					}
				}
			}
		}

		// We need to check if its a blueprint actor or not, as we handle blueprint root component
		// names a little bit differently
		bool bIsBlueprintGeneratedClass = ((UObject*)TargetActor->GetClass())->IsA(UBlueprintGeneratedClass::StaticClass());

		// Build component path up to TargetActor, if our Object is actually a SceneComponent
		USceneComponent* ComponentPointer = LeafComponent;
		while (ComponentPointer)
		{
			USceneComponent* AttachParent = ComponentPointer->GetAttachParent();

			FString ComponentName;

			// We're some form of root component, naming is different
			if (AttachParent == nullptr || AttachParent->GetOwner() != TargetActor)
			{
				// Users can rename of the root component for a blueprint generated class, so lets use that.
				// On other cases, users can't rename root components, and their actual names are always
				// something like StaticMeshComponent0 or LightComponent0 (even if its class is a UPointLightComponent).
				// Getting the class display name matches how the Variant Manager behaves
				ComponentName = bIsBlueprintGeneratedClass ?
					ComponentPointer->GetName() :
					ComponentPointer->GetClass()->GetDisplayNameText().ToString();

				ComponentPointer = nullptr;
			}
			else
			{
				ComponentName = ComponentPointer->GetName();
				ComponentPointer = AttachParent;
			}

			ComponentPath = ComponentName + PATH_DELIMITER + ComponentPath;
		}

		FString PropertyPath;

		FString ChildPropertyName = ChildProperty ? ChildProperty->GetDisplayNameText().ToString() : FString();
		static const TSet<FString> ProxyPropertyPaths{TEXT("Relative Location"), TEXT("Relative Rotation"), TEXT("Relative Scale 3D")};

		// We capture as just 'Material' in the Variant Manager UI, instead of 'Override Materials'
		// Override Materials doesn't work like a regular UArrayProperty, we need to use GetNumMaterials
		if (ChildProperty == FVariantManagerUtils::GetOverrideMaterialsProperty())
		{
			if (UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(LeafComponent))
			{
				// We'll capture all array indices
				PropertyPath += TEXT("Material");
			}
		}
		// Some properties are exposed on the actors, but are actually just proxies for the root component. Here we redirect those
		// to be on the root component itself, so as to remove duplicates
		else if (ComponentPath.IsEmpty() && ProxyPropertyPaths.Contains(ChildPropertyName))
		{
			FString RootComponentName = bIsBlueprintGeneratedClass ?
				TargetActor->GetRootComponent()->GetName() :
				TargetActor->GetRootComponent()->GetClass()->GetDisplayNameText().ToString();

			ComponentPath = RootComponentName + PATH_DELIMITER;
			PropertyPath += ChildPropertyName;
		}
		else
		{
			PropertyPath += ChildPropertyName;
		}

		Result.TargetActor = TargetActor;
		Result.LeafComponent = LeafComponent;
		Result.PathActorToLeafComponent = ComponentPath;
		Result.PathTailProperty = PropertyPath;
		return Result;
	}
}

void SVariantManager::OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event)
{
	if (!bAutoCaptureProperties || !Object || !Event.Property)
		{
		return;
		}

	bool bIsStructProperty = Event.MemberProperty && Event.MemberProperty->IsA(FStructProperty::StaticClass());
	bool bIsBuiltIn = bIsStructProperty && FVariantManagerUtils::IsBuiltInStructProperty(Event.MemberProperty);
	FProperty* Prop = bIsBuiltIn? Event.MemberProperty : Event.Property;

	FString PropertyPath;
	AActor* TargetActor = nullptr;

	// Fetch property path from the cache we build in OnPreObjectPropertyChanged
	for (int32 Index = CachedPropertyPathStack.Num() - 1; Index >= 0; --Index)
	{
		const SVariantManager::FCachedPropertyPath& CachedPropertyPath = CachedPropertyPathStack[Index];

		if (Object == CachedPropertyPath.Object &&
			Event.Property == CachedPropertyPath.ChildProperty &&
			Event.MemberProperty == CachedPropertyPath.ParentProperty)
		{
			PropertyPath = CachedPropertyPath.Path;
			TargetActor = CachedPropertyPath.TargetActor;
			CachedPropertyPathStack.RemoveAt(Index);
			break;
		}
	}

	// If its not in the cache, the best we got is to try and reconstruct the path with just ReconstructPropertyPath
	if (PropertyPath.IsEmpty())
	{
		SVariantManagerUtils::FReconstructionResult Result = SVariantManagerUtils::ReconstructPropertyPath(Object, Event.MemberProperty, Event.Property);
		if (Result.TargetActor == nullptr)
		{
			UE_LOGF(LogVariantManager, Warning, "Failed to auto-expose property '%ls' for object '%ls'", Event.MemberProperty? *Event.MemberProperty->GetDisplayNameText().ToString() : TEXT("nullptr"), Object ? *Object->GetName() : TEXT("nullptr"));
			return;
		}

		PropertyPath = Result.PathActorToLeafComponent + Result.PathTailProperty;
		TargetActor = Result.TargetActor;
	}

	if (PropertyPath.IsEmpty())
		{
		return;
		}

	bool bUpdatedSomething = AutoCaptureProperty(VariantManagerPtr.Pin().Get(), TargetActor, PropertyPath, Prop);
	if (bUpdatedSomething)
	{
		RefreshActorList();
	}
}

namespace SVariantManagerUtils
{
	int32 HashObjectAndPropChain(UObject* Object, const class FEditPropertyChain& PropChain)
	{
		using PropNode = TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode;

		int32 Hash = 7;
		for (PropNode* Node = PropChain.GetHead(); Node; Node = Node->GetNextNode())
		{
			Hash = HashCombine(Hash, GetTypeHash(Node->GetValue()));
		}

		Hash = HashCombine(Hash, GetTypeHash(Object));

		return Hash;
	}
}

void SVariantManager::OnPreObjectPropertyChanged(UObject* Object, const class FEditPropertyChain& PropChain)
{
	// For BP actors, Object will be the USceneComponent, and the path will start off from that component
	// For regular actors, it will fire twice. In both, the path starts from root component and goes to leaf,
	// but in the first firing the root component is Object, and in the second firing, the root component is

	if (!bAutoCaptureProperties || !Object)
	{
		return;
	}

	int32 Hash = SVariantManagerUtils::HashObjectAndPropChain(Object, PropChain);

	FProperty* ParentProp = PropChain.GetActiveMemberNode()->GetValue();
	FProperty* ChildProp = PropChain.GetActiveNode()->GetValue();

	SVariantManager::FCachedPropertyPath& CachedPropertyPath = CachedPropertyPaths.FindOrAdd(Hash);
	FString& Path = CachedPropertyPath.Path;
	if (!CachedPropertyPath.Path.IsEmpty())
	{
		CachedPropertyPathStack.Push(CachedPropertyPath);
		return;
	}

	using PropNode = TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode;
	PropNode* Head = PropChain.GetHead();
	if (!Head || !Head->GetValue())
	{
		return;
	}

	PropNode* Tail = PropChain.GetTail();
	if (!Tail || !Tail->GetValue())
	{
		return;
	}

	SVariantManagerUtils::FReconstructionResult Reconstruction = SVariantManagerUtils::ReconstructPropertyPath(Object, ParentProp, ChildProp);

	// Our first property may be a "shortcut" component pointer, so let's skip it
	while (Head && Reconstruction.LeafComponent)
	{
		FProperty* HeadProperty = Head->GetValue();

		bool bHeadPropertyInLeafComponent = false;
		for (TFieldIterator<FProperty> PropertyIterator(Reconstruction.LeafComponent->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			if (*PropertyIterator == HeadProperty)
			{
				bHeadPropertyInLeafComponent = true;
				break;
			}
		}

		if (bHeadPropertyInLeafComponent)
		{
			break;
		}

		Head = Head->GetNextNode();
	}

	// Component path is actually complete, lets travel down the property path.
	// Note how we don't visit the tail: The leaf property is processed in ReconstructPropertyPath already!
	bool bShowCategories = false;
	FString IntermediatePath = PATH_DELIMITER;
	for (PropNode* Node = Head; Node; Node = Node->GetNextNode())
	{
		FProperty* Prop = Node->GetValue();
		if (!Prop)
		{
			return;
		}

		FString PropertyName = Prop->GetDisplayNameText().ToString();

		// If we're a property inside a Struct, show the category
		if (bShowCategories)
		{
			FString Category = Prop->GetMetaData(TEXT("Category"));
			if (!Category.IsEmpty())
			{
				Category = Category.Replace(TEXT("|"), PATH_DELIMITER);

				int32 LastDelimiterIndex = Category.Find(PATH_DELIMITER, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				FString LastCategorySegment = (LastDelimiterIndex == INDEX_NONE) ? Category : Category.RightChop(LastDelimiterIndex + 1);

				if (!IntermediatePath.EndsWith(PATH_DELIMITER + Category + PATH_DELIMITER) &&
					LastCategorySegment != PropertyName)
				{
					IntermediatePath += Category + PATH_DELIMITER;
				}
			}
		}

		if (Node == Tail)
		{
			break;
		}

		IntermediatePath += PropertyName + PATH_DELIMITER;

		if (FStructProperty* StructProperty = CastField<FStructProperty>(Prop))
		{
			if (FVariantManagerUtils::IsBuiltInStructProperty(Prop))
			{
				// We don't care about anything deeper than this in this case
				Reconstruction.PathTailProperty.Empty();
				IntermediatePath.RemoveFromEnd(PATH_DELIMITER);
				break;
			}
			else if (FVariantManagerUtils::IsWalkableStructProperty(Prop))
			{
				bShowCategories = true;
			}
		}
		else
		{
			bShowCategories = false;
		}
	}
	IntermediatePath.RemoveFromStart(PATH_DELIMITER);

	CachedPropertyPath.Object = Object;
	CachedPropertyPath.ParentProperty = ParentProp;
	CachedPropertyPath.ChildProperty = ChildProp;
	CachedPropertyPath.TargetActor = Reconstruction.TargetActor;
	CachedPropertyPath.Path = Reconstruction.PathActorToLeafComponent + IntermediatePath + Reconstruction.PathTailProperty;

	CachedPropertyPathStack.Push(CachedPropertyPath);
}

void SVariantManager::OnPieEvent(bool bIsSimulating)
{
	// We must forcebly clear these, because during PIE the actors/components remain
	// alive in the editor world, meaning UPropertyValues::HasValidResolve() will return true.
	// Ideally they would subscribe to that event themselves, but that would require
	// VariantManagerContent depend on the Editor module
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (PinnedVariantManager.IsValid())
	{
		if (ULevelVariantSets* LVS = PinnedVariantManager->GetCurrentLevelVariantSets())
		{
			for (UVariantSet* VarSet : LVS->GetVariantSets())
			{
				for (UVariant* Var : VarSet->GetVariants())
				{
					for (UVariantObjectBinding* Binding : Var->GetBindings())
					{
						for (UPropertyValue* Prop : Binding->GetCapturedProperties())
						{
							Prop->ClearLastResolve();
						}
					}
				}
			}
		}
	}

	CachedPropertyPaths.Empty();
	CachedPropertyPathStack.Empty();

	CachedAllActorPaths.Empty();
	RefreshActorList();
}

void SVariantManager::ReorderPropertyNodes(const TArray<TSharedPtr<FVariantManagerPropertyNode>>& TheseNodes, TSharedPtr<FVariantManagerPropertyNode> Pivot, EItemDropZone RelativePosition)
{
	FScopedTransaction Transaction(LOCTEXT("ReorderedPropertyTransaction", "Reorder property or function captures"));

	// Remove them from their current position
	for (const TSharedPtr<FVariantManagerPropertyNode>& ThisNode : TheseNodes)
	{
		DisplayedPropertyNodes.Remove(ThisNode);
	}

	// Insert them into their new position
	int32 Index = DisplayedPropertyNodes.IndexOfByKey(Pivot);
	Index += RelativePosition == EItemDropZone::BelowItem ? 1 : 0;
	DisplayedPropertyNodes.Insert(TheseNodes, Index);

	// Normalize the DisplayOrder to an increasing count
	uint32 Order = 0;
	for (TSharedPtr<FVariantManagerPropertyNode>& DisplayedNode : DisplayedPropertyNodes)
	{
		if (DisplayedNode.IsValid())
		{
			DisplayedNode->SetDisplayOrder(Order++);
		}
	}

	RefreshPropertyList();
}

TSharedRef<ITableRow> SVariantManager::GenerateRightTreeRow( TSharedRef<ERightTreeRowType> RowType, const TSharedRef<STableViewBase>& OwnerTable )
{
	TSharedRef<STableRow<TSharedRef<ERightTreeRowType>>> TableRow = SNew( STableRow<TSharedRef<ERightTreeRowType>>, OwnerTable );

	TSharedPtr<SWidget> RowContent = SNullWidget::NullWidget;
	switch ( *RowType )
	{
	case ERightTreeRowType::DependenciesHeader:
		RowContent = GenerateRightTreeHeaderRowContent(*RowType, TableRow);
		break;
	case ERightTreeRowType::PropertiesContent:
		RowContent = GenerateRightTreePropertiesRowContent();
		break;
	case ERightTreeRowType::DependenciesContent:
		RowContent = GenerateRightTreeDependenciesRowContent();
		break;
	default:
		break;
	}

	TableRow->SetRowContent(RowContent.ToSharedRef());
	return TableRow;
}

TSharedRef<SWidget> SVariantManager::GenerateRightTreeHeaderRowContent( ERightTreeRowType RowType, TSharedRef<STableRow<TSharedRef<ERightTreeRowType>>> InTableRow )
{
	const bool bUseNewAssetPicker = UE::VariantManager::Get_CVar_VariantManager_AssetPicker().GetValueOnGameThread();

	return SNew(SBox)
		[
			SNew(SBorder)
			.VAlign( VAlign_Center )
			.HAlign(HAlign_Fill)
			.BorderImage(&HeaderStyle->NormalBrush)
			.Padding(FMargin(10, 7, 8, 8))
			[
				SNew(SHorizontalBox)
				.IsEnabled_Lambda([this]()
				{
					TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
					if (PinnedVariantManager.IsValid())
					{
						TArray<UVariant*> SelectedVariants;
						TArray<UVariantSet*> SelectedVariantSets;
						PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);

						return SelectedVariants.Num() == 1;
					}
					return false;
				})
			
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SExpanderArrow, InTableRow).IndentAmount(FVariantManagerStyle::Get().GetFloat("VariantManager.Spacings.IndentAmount"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				[
					bUseNewAssetPicker ?
						SNew(SRichTextBlock)
						.Text(LOCTEXT("Dependencies", "Dependencies"))
					:
						SNew(SRichTextBlock)
						.Text(LOCTEXT("Dependencies", "Dependencies"))
						.DecoratorStyleSet(&FAppStyle::Get())
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.OnClicked(this, &SVariantManager::OnAddDependencyClicked)
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("AddDependencyTooltip", "Add a new dependency to this variant"))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(24.0f)
					.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
					[
						SNew(SBox)
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.OnClicked(this, &SVariantManager::OnDeleteSelectedDependencies)
							.ContentPadding(FMargin(1, 0))
							.ToolTipText(LOCTEXT("DeleteDependency", "Delete selected dependencies"))
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]
				]
			]
		];
}

FReply SVariantManager::OnDeleteSelectedDependencies()
{
	TArray<FVariantDependencyModelPtr> SelectedItems = DependenciesList->GetSelectedItems();
	
	if (SelectedItems.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteDependencies", "Delete variant dependencies"));

		for (FVariantDependencyModelPtr Item : SelectedItems)
		{
			UVariant* ParentVariant = Item->ParentVariant.Get();
			if (Item->Dependency && ParentVariant)
			{
				int32 DependencyIndex = INDEX_NONE;
				for (int32 Index = 0; Index < ParentVariant->GetNumDependencies(); ++Index)
				{
					if (&ParentVariant->GetDependency(Index) == Item->Dependency)
					{
						DependencyIndex = Index;
						break;
					}
				}

				if (DependencyIndex != INDEX_NONE)
				{


					ParentVariant->DeleteDependency(DependencyIndex);
				}
			}
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SVariantManager::GenerateRightTreePropertiesRowContent()
{
	float BorderThickness = FVariantManagerStyle::Get().GetFloat("VariantManager.Spacings.BorderThickness");

	const bool bShowColumns = UE::VariantManager::Get_CVar_VariantManager_AssetPicker().GetValueOnAnyThread();

	TSharedPtr<SWidget> PropertyListWidget;

	float ActorsSplitterPrecent = 0.5f;
	float PropertiesSplitterPercent = 0.5f;

	if (bShowColumns)
	{
		ActorsSplitterPrecent = 0.2f;
		PropertiesSplitterPercent = 0.8f;

		PropertyListWidget = SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(CapturedPropertyListView, SListView<TSharedPtr<FVariantManagerPropertyNode>>)
					.SelectionMode(ESelectionMode::Single)
					.ListItemsSource(&DisplayedPropertyNodes)
					.OnContextMenuOpening(this, &SVariantManager::OnPropertyListContextMenuOpening)
					.OnGenerateRow_Static(&SVariantManager::MakeCapturedPropertyRowWithColumns)
					.Visibility(EVisibility::Visible)
					.HeaderRow(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(SVariantManagerMultiColumnPropertyTableRow::ColumnPath)
						.DefaultLabel(LOCTEXT("PathColumnHeader", "Path"))
						.SortMode(this, &SVariantManager::GetSortMode, SVariantManagerMultiColumnPropertyTableRow::ColumnPath)
						.SortPriority(this,  &SVariantManager::GetSortPriority, SVariantManagerMultiColumnPropertyTableRow::ColumnPath)
						.OnSort(this, &SVariantManager::HandleSort)
						.FillWidth(0.2f)

						+ SHeaderRow::Column(SVariantManagerMultiColumnPropertyTableRow::ColumnProperty)
						.DefaultLabel(LOCTEXT("PropertyColumnHeader", "Property"))
						.SortMode(this, &SVariantManager::GetSortMode, SVariantManagerMultiColumnPropertyTableRow::ColumnProperty)
						.SortPriority(this,  &SVariantManager::GetSortPriority, SVariantManagerMultiColumnPropertyTableRow::ColumnProperty)
						.OnSort(this, &SVariantManager::HandleSort)
						.FillWidth(0.3f)

						+ SHeaderRow::Column(SVariantManagerMultiColumnPropertyTableRow::ColumnValue)
						.DefaultLabel(LOCTEXT("ValueColumnHeader", "Value"))
						.FillWidth(0.5f)
					)
			];
	}
	else
	{
		PropertyListWidget = SAssignNew(CapturedPropertyListView, SListView<TSharedPtr<FVariantManagerPropertyNode>>)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&DisplayedPropertyNodes)
			.OnContextMenuOpening(this, &SVariantManager::OnPropertyListContextMenuOpening)
			.OnGenerateRow_Static(&SVariantManager::MakeCapturedPropertyRow)
			.Visibility(EVisibility::Visible);
	}


	return SNew( SBox )
	[
		// Actor column
		SAssignNew( PropertiesSplitter, SSplitter )
		.Orientation( Orient_Horizontal )
		.PhysicalSplitterHandleSize( BorderThickness )
		.HitDetectionSplitterHandleSize( BorderThickness )

		+ SSplitter::Slot()
		.Value(ActorsSplitterPrecent)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(&HeaderStyle->NormalBrush)
				.Padding(FMargin(10, 7, 8, 8))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ActorsText", "Actors"))
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.OnClicked(this, &SVariantManager::OnSummonAddActorMenu)
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("AddActorPlusTooltip", "Add a new actor binding to selected variants"))
						.IsEnabled_Lambda([&VariantManagerPtr = VariantManagerPtr]() -> bool
						{
							if (TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin())
							{
								TArray<UVariant*> Variants;
								TArray<UVariantSet*> VariantSets;
								VariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Variants, VariantSets);

								if (Variants.Num() > 0)
								{
									return true;
								}

								for (const UVariantSet* Set : VariantSets)
								{
									if (Set && Set->GetNumVariants() > 0)
									{
										return true;
									}
								}
							}
							return false;
						})
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			[
				SNew(SBox)
				[
					ActorListView.ToSharedRef()
				]
			]
		]

		// Properties column
		+ SSplitter::Slot()
		.Value(PropertiesSplitterPercent)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(&HeaderStyle->NormalBrush)
				.Padding(FMargin(10, 7, 8, 8))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PropertiesText", "Properties"))
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ContentPadding(FMargin(1, 0))
						.ToolTipText(LOCTEXT("CapturePropertiesPlusTooltip", "Capture properties from the selected actor bindings"))
						.OnClicked_Lambda([this]
						{
							CaptureNewPropertiesFromSelectedActors();
							return FReply::Handled();
						})
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						.IsEnabled_Lambda([&VariantManagerPtr = VariantManagerPtr]() -> bool
						{
							if (TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin())
							{
								return VariantManager->GetSelection().GetSelectedActorNodes().Num() > 0;
							}
							return false;
						})
					]
				]
				
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			[
				PropertyListWidget.ToSharedRef()
			]
		]
	];
}

TSharedRef<SWidget> SVariantManager::GenerateRightTreeDependenciesRowContent()
{
	return 
		SNew( SBox )
		[
			SNew(SVerticalBox)
			.IsEnabled_Lambda([this]()
			{
				TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
				if (PinnedVariantManager.IsValid())
				{
					TArray<UVariant*> SelectedVariants;
					TArray<UVariantSet*> SelectedVariantSets;
					PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);

					return SelectedVariants.Num() == 1;
				}
				return false;
			})

			+ SVerticalBox::Slot()
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Horizontal)
				.Thickness(3)
				.SeparatorImage(&SplitterStyle->HandleNormalBrush)
			]

			// Dependencies
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SBox) // Very important to prevent the tree from expanding freely
				[
					SAssignNew(DependenciesList, SListView<FVariantDependencyModelPtr>)
					.SelectionMode(ESelectionMode::Multi)
					.ListItemsSource(&DisplayedDependencies)
					.OnGenerateRow(this, &SVariantManager::GenerateDependencyRow, true)
					.HeaderRow(SNew(SHeaderRow)
						+ SHeaderRow::Column(SDependencyRow::VisibilityColumn)
						.DefaultLabel(FText())
						.FixedWidth(30)
						+ SHeaderRow::Column(SDependencyRow::VariantSetColumn)
						.DefaultLabel(LOCTEXT("DependenciesVariantSetText", "Variant Set"))
						.FillWidth(0.5f)
						+ SHeaderRow::Column(SDependencyRow::VariantColumn)
						.DefaultLabel(LOCTEXT("DependenciesVariantText", "Variant"))
						.FillWidth(0.5f))
					.Visibility(EVisibility::Visible)
				]
			]
		];
}

TSharedRef<ITableRow> SVariantManager::GenerateDependencyRow(FVariantDependencyModelPtr Dependency, const TSharedRef<STableViewBase>& OwnerTable, bool bInteractionEnabled)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return SNew(STableRow<FVariantDependencyModelPtr>, OwnerTable);
	}

	TArray<UVariant*> Variants;
	TArray<UVariantSet*> VariantSets;
	PinnedVariantManager->GetSelection().GetSelectedVariantsAndVariantSets(Variants, VariantSets);
	if (Variants.Num() != 1 || Variants[0] == nullptr )
	{
		return SNew(STableRow<FVariantDependencyModelPtr>, OwnerTable);
	}

	return SNew(SDependencyRow, OwnerTable, Dependency, bInteractionEnabled)
		.IsEnabled(!Dependency->bIsDependent && !Dependency->bIsDivider);
}

void SVariantManager::OnEditorSelectionChanged(UObject* NewSelection)
{
	if (!bRespondToEditorSelectionEvents)
	{
		return;
	}

	USelection* Selection = Cast<USelection>(NewSelection);
	if (!Selection)
	{
		return;
	}

	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid())
	{
		return;
	}

	FVariantManagerSelection& VariantSelection = PinnedVariantManager->GetSelection();

	TArray<UObject*> SelectedActors;
	Selection->GetSelectedObjects(AActor::StaticClass(), SelectedActors);
	TSet<UObject*> SelectedActorsSet{SelectedActors};

	TArray<UActorComponent*> SelectedComponents;
	Selection->GetSelectedObjects<UActorComponent>(SelectedComponents);
	for (UActorComponent* Component : SelectedComponents)
	{
		SelectedActorsSet.Add(Component->GetOwner());
	}

	SelectedActorsSet.Remove(nullptr);

	// No point in clearing our selection the editor doesn't have anything selected either
	// Sometimes empty selection events seem to fire when selecting the actual actor "row" on its
	// component tree display
	if (SelectedActorsSet.Num() == 0)
	{
		return;
	}

	VariantSelection.SuspendBroadcast();
	for (TSharedRef<FVariantManagerDisplayNode>& DisplayedActor : DisplayedActors)
	{
		if (DisplayedActor->GetType() != EVariantManagerNodeType::Actor)
		{
			continue;
		}

		TSharedRef<FVariantManagerActorNode> ActorNode = StaticCastSharedRef<FVariantManagerActorNode>(DisplayedActor);
		TWeakObjectPtr<UVariantObjectBinding> ObjectBinding = ActorNode->GetObjectBinding();
		if (!ObjectBinding.IsValid())
		{
			continue;
		}

		UObject* BindingObject = ObjectBinding->GetObject();
		if (BindingObject && SelectedActorsSet.Contains(BindingObject))
		{
			VariantSelection.AddActorNodeToSelection(ActorNode);
		}
		else
		{
			VariantSelection.RemoveActorNodeFromSelection(ActorNode);
		}
	}
	VariantSelection.ResumeBroadcast();

	RefreshActorList();
}

void SVariantManager::OnThumbnailChanged(UObject* VariantOrVariantSet)
{
	TSharedPtr<FVariantManager> PinnedVariantManager = VariantManagerPtr.Pin();
	if (!PinnedVariantManager.IsValid() || !VariantOrVariantSet)
	{
		return;
	}

	if (VariantOrVariantSet->GetOutermost() == PinnedVariantManager->GetCurrentLevelVariantSets()->GetOutermost())
	{
		RefreshVariantTree();
	}
}

EColumnSortMode::Type SVariantManager::GetSortMode(const FName ColumnName) const
{
	return ColumnName == SortColumn ? SortMode : EColumnSortMode::None;
}

EColumnSortPriority::Type SVariantManager::GetSortPriority(const FName ColumnName) const
{
	return ColumnName == SortColumn ? SortPriority : EColumnSortPriority::None;
}

void SVariantManager::HandleSort(
	const EColumnSortPriority::Type NewSortPriority, 
	const FName& ColumnId,
	const EColumnSortMode::Type NewSortMode)
{
	if (NewSortPriority == EColumnSortPriority::Primary)
	{
		SortColumn = ColumnId;
		SortMode = NewSortMode;
		SortPriority = NewSortPriority;
	}

	SortItems();

	if (CapturedPropertyListView)
	{
		CapturedPropertyListView->RequestListRefresh();
	}
}

void SVariantManager::SortItems()
{
	auto SortFn = [this](const TSharedPtr<FVariantManagerPropertyNode>& A, const TSharedPtr<FVariantManagerPropertyNode>& B)
		{
			if (SortColumn == SVariantManagerMultiColumnPropertyTableRow::ColumnPath)
			{
				if (A->GetSubPath() != B->GetSubPath())
				{
					const bool bRet = A->GetSubPath() < B->GetSubPath();

					return SortMode == EColumnSortMode::Ascending ? bRet : !bRet;
				}
			}
			else if (SortColumn == SVariantManagerMultiColumnPropertyTableRow::ColumnProperty)
			{
				if (A->GetPropertyName() != B->GetPropertyName())
				{
					const bool bRet = A->GetPropertyName() < B->GetPropertyName();

					return SortMode == EColumnSortMode::Ascending ? bRet : !bRet;
				}
			}

			return false;
		};
	DisplayedPropertyNodes.Sort(SortFn);
}

#undef VM_COMMON_PADDING
#undef VM_COMMON_HEADER_MAX_HEIGHT
#undef LOCTEXT_NAMESPACE

