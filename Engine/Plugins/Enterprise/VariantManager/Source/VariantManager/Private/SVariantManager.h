// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

class FUICommandList;
class SScrollBar;
class UVariantObjectBinding;
class UVariantSet;
enum class EItemDropZone;
struct FSplitterStyle;
struct FTableColumnHeaderStyle;
template <typename ItemType> class SListView;
template <typename ItemType> class STableRow;

class FExtender;
class FTransactionObjectEvent;
class FVariantManager;
class FVariantManagerDisplayNode;
class FVariantManagerPropertyNameNode;
class FVariantManagerPropertyNode;
class ITableRow;
class SSplitter;
class STableViewBase;
class SVariantManagerActorListView;
class SVariantManagerNodeTreeView;
class UVariant;
enum class EMapChangeType : uint8;
struct FSlateImageBrush;
struct FVariantDependency;
struct FVariantDependencyModel;
using FVariantDependencyModelPtr = TSharedPtr<FVariantDependencyModel>;

// Convenience struct to save/load how the user configured the main splitters
struct FSplitterValues
{
	float PropertyNameColumn = 0.5f;
	float PropertyValueColumn = 0.5f;
	float VariantColumn = 0.25f;

	FSplitterValues(){};
	FSplitterValues(FString& InSerialized);
	FString ToString();
};

struct FVariantManagerPropertiesColumnSizeData
{
	TAttribute<float> NameColumnWidth;
	TAttribute<float> ValueColumnWidth;
	SSplitter::FOnSlotResized OnSplitterNameColumnChanged;
	SSplitter::FOnSlotResized OnSplitterValueColumnChanged;
};

class SVariantManager
	: public SCompoundWidget
	, public FNotifyHook
{
public:

	DECLARE_DELEGATE_OneParam(FOnToggleBoolOption, bool)
	SLATE_BEGIN_ARGS(SVariantManager)
	{ }
		/** Extender to use for the add menu. */
		SLATE_ARGUMENT(TSharedPtr<FExtender>, AddMenuExtender)

		/** Extender to use for the toolbar. */
		SLATE_ARGUMENT(TSharedPtr<FExtender>, ToolbarExtender)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FVariantManager> InVariantManager);
	~SVariantManager();

	void CreateCommandBindings();

	TSharedPtr<FUICommandList> GetVariantTreeCommandBindings() const
	{
		return VariantTreeCommandBindings;
	}

	TSharedPtr<FUICommandList> GetActorListCommandBindings() const
	{
		return ActorListCommandBindings;
	}

	TSharedPtr<FUICommandList> GetPropertyListCommandBindings() const
	{
		return PropertyListCommandBindings;
	}

	// Commands
	void AddEditorSelectedActorsToVariant();
	bool CanAddEditorSelectedActorsToVariant();

	void CreateNewVariantSet();
	bool CanCreateNewVariantSet();

	void CutSelectionVariantTree();
	void CopySelectionVariantTree();
	void PasteSelectionVariantTree();
	void DeleteSelectionVariantTree();
	void DuplicateSelectionVariantTree();
	void RenameSelectionVariantTree();

	bool CanCutVariantTree();
	bool CanCopyVariantTree();
	bool CanPasteVariantTree();
	bool CanDeleteVariantTree();
	bool CanDuplicateVariantTree();
	bool CanRenameVariantTree();

	void CutSelectionActorList();
	void CopySelectionActorList();
	void PasteSelectionActorList();
	void DeleteSelectionActorList();
	void DuplicateSelectionActorList();
	void RenameSelectionActorList();

	bool CanCutActorList();
	bool CanCopyActorList();
	bool CanPasteActorList();
	bool CanDeleteActorList();
	bool CanDuplicateActorList();
	bool CanRenameActorList();

	void SwitchOnSelectedVariant();
	void CreateThumbnail();
	void SelectThumbnailAsset();
	void LoadThumbnail();
	void ClearThumbnail();

	bool CanSwitchOnVariant();
	bool CanCreateThumbnail();
	bool CanLoadThumbnail();
	bool CanClearThumbnail();

	void CaptureNewPropertiesFromSelectedActors();
	bool CanCaptureNewPropertiesFromSelectedActors();

	void AddFunctionCaller();
	bool CanAddFunctionCaller();

	void RebindToSelectedActor();
	bool CanRebindToSelectedActor();

	void RemoveActorBindings();
	bool CanRemoveActorBindings();

	void ApplyProperty();
	void RecordProperty();
	void RemoveCapture();
	void CallDirectorFunction();
	void RemoveDirectorFunctionCaller();

	bool CanApplyProperty();
	bool CanRecordProperty();
	bool CanRemoveCapture();
	bool CanCallDirectorFunction();
	bool CanRemoveDirectorFunctionCaller();

	void SwitchOnVariant(UVariant* Variant);

	void GetSelectedBindingAndEditorActor(UVariantObjectBinding*& OutSelectedBinding, UObject*& OutSelectedObject);

	// Sorts display nodes based on their order on the screen
	// Can be used to sort selected nodes
	void SortDisplayNodes(TArray<TSharedRef<FVariantManagerDisplayNode>>& DisplayNodes);

	TSharedRef<SWidget> MakeAddButton();
	TSharedRef<SWidget> GetMenuContextForLevelVariantSets();
	TSharedRef<SWidget> MakeLevelVariantSetsComboButton();

	FVariantManagerPropertiesColumnSizeData& GetPropertiesColumnSizeData()
	{
		return PropertiesColumnSizeData;
	}

	static TSharedRef<ITableRow> MakeCapturedPropertyRow(TSharedPtr<FVariantManagerPropertyNode> Item, const TSharedRef<STableViewBase>& OwnerTable);
	static TSharedRef<ITableRow> MakeCapturedPropertyRowWithColumns(TSharedPtr<FVariantManagerPropertyNode> Item, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<SWidget> OnPropertyListContextMenuOpening() const;

	void OnOutlinerNodeSelectionChanged();
	void OnActorNodeSelectionChanged();
	void OnVariantDependenciesUpdated(UVariant* ParentVariant);

	// These completely refresh the data and the view for each display
	void RefreshVariantTree();
	void RefreshActorList();
	void RefreshPropertyList();
	void RefreshDependencyLists();
	void UpdatePropertyDefaults();

	void OnBlueprintCompiled();
	void OnMapChanged(UWorld* World, EMapChangeType MapChangeType);

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	void OnOutlinerSearchChanged(const FText& Filter);

	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual void OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent ) override;

	FReply OnAddVariantSetClicked();
	FReply OnSummonAddActorMenu();
	FReply OnAddDependencyClicked();

	FReply OnDeleteSelectedDependencies();

	// Callbacks for property ColumnSizeData
	float OnGetPropertiesNameColumnWidth() const { return PropertiesNameColumnWidth; }
	float OnGetPropertiesValueColumnWidth() const { return PropertiesValueColumnWidth; }

	void OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& Event);
	void OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event);
	void OnPreObjectPropertyChanged(UObject* Object, const class FEditPropertyChain& PropChain);
	void OnPieEvent(bool bIsSimulating);
	void OnEditorSelectionChanged(UObject* NewSelection);

	void OnThumbnailChanged(UObject* VariantOrVariantSet);

	void ReorderPropertyNodes(const TArray<TSharedPtr<FVariantManagerPropertyNode>>& TheseNodes, TSharedPtr<FVariantManagerPropertyNode> Pivot, EItemDropZone RelativePosition);

private:
	enum class ERightTreeRowType
	{
		PropertiesHeader,
		PropertiesContent,
		DependenciesHeader,
		DependenciesContent
	};

	TSharedRef<ITableRow> GenerateRightTreeRow( TSharedRef<ERightTreeRowType> RowType, const TSharedRef<STableViewBase>& OwnerTable );
	TSharedRef<SWidget> GenerateRightTreeHeaderRowContent( ERightTreeRowType RowType, TSharedRef<STableRow<TSharedRef<ERightTreeRowType>>> InTableRow );
	TSharedRef<SWidget> GenerateRightTreePropertiesRowContent();
	TSharedRef<SWidget> GenerateRightTreeDependenciesRowContent();
	TSharedRef<ITableRow> GenerateDependencyRow( FVariantDependencyModelPtr Dependency, const TSharedRef<STableViewBase>& OwnerTable, bool bInteractionEnabled );

	// Set thumbnail of variant set, if it does not have one, to be the same as first found
	// variant thumbnail
	void UpdateVariantSetThumbnail(UVariantSet* InVariantSet);

	void OnAssetDroppedForVariantManager(const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> AssetDatas);
	static bool OnAreAssetsAcceptableForVariantManager(TArrayView<FAssetData> Assets, FText& OutReason);

	static FReply OnUseSelectedAssetFromContentBrowserForCurrentLevelVariantSet();
	FReply OnBrowseToCurrentLevelVariantSet() const;

	EColumnSortMode::Type GetSortMode(const FName ColumnName) const;
	EColumnSortPriority::Type GetSortPriority(const FName ColumnName) const;
	void HandleSort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	void SortItems();

private:
	TWeakPtr<FVariantManager> VariantManagerPtr;

	TSharedPtr<SVariantManagerNodeTreeView> NodeTreeView;

	TSharedPtr<SVariantManagerActorListView> ActorListView;
	TArray<TSharedRef<FVariantManagerDisplayNode>> DisplayedActors;

	TSharedPtr<SListView<TSharedPtr<FVariantManagerPropertyNode>>> CapturedPropertyListView;
	TArray<TSharedPtr<FVariantManagerPropertyNode>> DisplayedPropertyNodes;

	TSharedPtr<SListView<FVariantDependencyModelPtr>> DependenciesList;
	TArray<FVariantDependencyModelPtr> DisplayedDependencies;

	// We use paths here to avoid having to check if the bindings are resolved
	TSet<FString> CachedSelectedActorPaths;
	TSet<FString> CachedDisplayedActorPaths;
	TSet<FString> CachedAllActorPaths;

	TSharedPtr<SScrollBar> ScrollBar;

	TArray<TSharedPtr<class IPropertyChangeListener>> PropertyChangeListeners;

	TSharedPtr<FUICommandList> VariantTreeCommandBindings;
	TSharedPtr<FUICommandList> ActorListCommandBindings;
	TSharedPtr<FUICommandList> PropertyListCommandBindings;

	bool bAutoCaptureProperties = false;

	FVariantManagerPropertiesColumnSizeData PropertiesColumnSizeData;
	float PropertiesNameColumnWidth = 0;
	float PropertiesValueColumnWidth = 0;

	FSplitterValues SplitterValues;

	FDelegateHandle OnObjectTransactedHandle;
	FDelegateHandle OnBlueprintCompiledHandle;
	FDelegateHandle OnMapChangedHandle;
	FDelegateHandle OnObjectPropertyChangedHandle;
	FDelegateHandle OnPreObjectPropertyChangedHandle;
	FDelegateHandle OnBeginPieHandle;
	FDelegateHandle OnEndPieHandle;
	FDelegateHandle OnEditorSelectionChangedHandle;
	FDelegateHandle OnVariantThumbnailUpdatedHandle;
	FDelegateHandle OnVariantSetThumbnailUpdatedHandle;

	// We keep track of this to remember splitter values between loads
	TSharedPtr<SSplitter> MainSplitter;
	TSharedPtr<SSplitter> PropertiesSplitter;
	TSharedPtr<SSplitter> DependenciesSplitter;

	TArray<TSharedRef<ERightTreeRowType>> RightTreeRootItems;

	struct FCachedPropertyPath
	{
		UObject* Object;
		FProperty* ParentProperty;
		FProperty* ChildProperty;
		AActor* TargetActor;
		FString Path;
	};

	// Structures used to optimize construction and usage of property paths related to auto-expose,
	// as we must use the pre- and post-property-changed events in combination
	TMap<int32, SVariantManager::FCachedPropertyPath> CachedPropertyPaths;
	TArray<SVariantManager::FCachedPropertyPath> CachedPropertyPathStack;

	bool bRespondToEditorSelectionEvents = true;

	const FTableColumnHeaderStyle* HeaderStyle = nullptr;
	const FSplitterStyle* SplitterStyle = nullptr;

	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;
	EColumnSortPriority::Type SortPriority = EColumnSortPriority::Primary;
	FName SortColumn;
};
