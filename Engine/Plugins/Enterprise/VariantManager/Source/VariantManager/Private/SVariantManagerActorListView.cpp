// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariantManagerActorListView.h"

#include "DisplayNodes/VariantManagerActorNode.h"
#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ScopedTransaction.h"
#include "SVariantManager.h"
#include "SVariantManagerTableRow.h"
#include "VariantManager.h"
#include "VariantManagerSelection.h"
#include "VariantSet.h"

#define LOCTEXT_NAMESPACE "SVariantManagerActorListView"

void SVariantManagerActorListView::Construct(const FArguments& InArgs, TWeakPtr<FVariantManager> InVariantManagerPtr)
{
	VariantManagerPtr = InVariantManagerPtr;
	bCanDrop = false;

	SListView::Construct
	(
		SListView::FArguments()
		.SelectionMode(ESelectionMode::Multi)
		.ListItemsSource(InArgs._ListItemsSource)
		.OnContextMenuOpening(this, &SVariantManagerActorListView::OnActorListContextMenuOpening)
		.OnGenerateRow(this, &SVariantManagerActorListView::OnGenerateActorRow)
		.Visibility(EVisibility::Visible)
	);
}

void SVariantManagerActorListView::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	bCanDrop = false;

	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (DecoratedDragDropOp.IsValid())
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
}

FReply SVariantManagerActorListView::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FActorDragDropGraphEdOp> SceneActorDragDropOp = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();

	TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
	if (!VarMan.IsValid())
	{
		return FReply::Unhandled();
	}

	if (SceneActorDragDropOp.IsValid())
	{
		FVariantManagerSelection& Selection = VarMan->GetSelection();

		// Get all unique selected variants
		TArray<UVariant*> SelectedVariants;
		TArray<UVariantSet*> SelectedVariantSets;
		Selection.GetSelectedVariantsAndVariantSets(SelectedVariants, SelectedVariantSets);
		for (UVariantSet* VarSet : SelectedVariantSets)
		{
			SelectedVariants.Append(VarSet->GetVariants());
		}
		TSet<UVariant*> UniqueVariants = TSet<UVariant*>(SelectedVariants);

		// Get all unique dragged actors that we can add
		TArray<TWeakObjectPtr<AActor>> DraggedActors = SceneActorDragDropOp->Actors;
		TSet<TWeakObjectPtr<AActor>> AllActorsWeCanAdd;

		int32 VariantsThatCanAccept = 0;
		for (UVariant* Var : UniqueVariants)
		{
			TArray<TWeakObjectPtr<AActor>> ActorsWeCanAdd;
			VarMan->CanAddActorsToVariant(DraggedActors, Var, ActorsWeCanAdd);

			if (ActorsWeCanAdd.Num() > 0)
			{
				VariantsThatCanAccept += 1;
				AllActorsWeCanAdd.Append(ActorsWeCanAdd);
			}
		}

		int32 NumActors = AllActorsWeCanAdd.Num();

		if (NumActors > 0)
		{
			FText NewHoverText = FText::Format( LOCTEXT("CanDropActors", "Bind {0} {0}|plural(one=actor,other=actors) to {1} selected {1}|plural(one=variant,other=variants)"),
				NumActors,
				VariantsThatCanAccept);

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);

			bCanDrop = true;

			return FReply::Handled();
		}
		else if (SelectedVariants.Num() > 0 || SelectedVariantSets.Num() > 0)
		{
			FText NewHoverText = FText( LOCTEXT("AllActorsAlreadyBound", "Actors already bound to all selected variants"));

			const FSlateBrush* NewHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

			DecoratedDragDropOp->SetToolTip(NewHoverText, NewHoverIcon);
		}
		else
		{
			DecoratedDragDropOp->ResetToDefaultToolTip();
		}
	}

	bCanDrop = false;

	return FReply::Unhandled();
}

FReply SVariantManagerActorListView::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if (!bCanDrop)
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FActorDragDropGraphEdOp> SceneActorDragDropOp = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
	if (SceneActorDragDropOp.IsValid())
	{
		TSharedPtr<FVariantManager> VarMan = VariantManagerPtr.Pin();
		if (VarMan.IsValid())
		{
			const TSet<TSharedRef<FVariantManagerDisplayNode>>& Nodes = VarMan->GetSelection().GetSelectedOutlinerNodes();

			// Get dragged actors
			TArray<TWeakObjectPtr<AActor>> DraggedActors = SceneActorDragDropOp->Actors;
			TArray<AActor*> DraggedRawActors;
			for (TWeakObjectPtr<AActor>& ActorWeakPtr : DraggedActors)
			{
				DraggedRawActors.Add(ActorWeakPtr.Get());
			}

			// Get all selected variants
			TArray<UVariant*> SelectedVariants;
			for (const TSharedRef<FVariantManagerDisplayNode>& Node : Nodes)
			{
				if (Node->GetType() == EVariantManagerNodeType::Variant)
				{
					TSharedPtr<FVariantManagerVariantNode> SomeNodeAsVariant = StaticCastSharedRef<FVariantManagerVariantNode>(Node);
					if (SomeNodeAsVariant.IsValid())
					{
						SelectedVariants.Add(&SomeNodeAsVariant->GetVariant());
					}
				}
			}

			FScopedTransaction Transaction(FText::Format(
				LOCTEXT("ActorListViewDropVariants", "Drop {0} scene {0}|plural(one=actor,other=actors) on {1} {1}|plural(one=variant,other=variants)"),
				DraggedRawActors.Num(),
				SelectedVariants.Num()));

			VarMan->CreateObjectBindingsAndCaptures(DraggedRawActors, SelectedVariants);

			VarMan->GetVariantManagerWidget()->RefreshActorList();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SVariantManagerActorListView::Private_SetItemSelection(TSharedRef<FVariantManagerDisplayNode> TheItem, bool bShouldBeSelected, bool bWasUserDirected)
{
	SListView::Private_SetItemSelection(TheItem, bShouldBeSelected, bWasUserDirected);
	UpdateSelectionFromListView();
}

void SVariantManagerActorListView::Private_ClearSelection()
{
	SListView::Private_ClearSelection();
	UpdateSelectionFromListView();
}

void SVariantManagerActorListView::Private_SelectRangeFromCurrentTo(TSharedRef<FVariantManagerDisplayNode> InRangeSelectionEnd)
{
	SListView::Private_SelectRangeFromCurrentTo(InRangeSelectionEnd);
	UpdateSelectionFromListView();
}

void SVariantManagerActorListView::Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo)
{
	SListView::Private_SignalSelectionChanged(SelectInfo);

	FVariantManagerSelection& Selection = VariantManagerPtr.Pin()->GetSelection();
	Selection.RequestActorNodeSelectionChangedBroadcast();
}

void SVariantManagerActorListView::UpdateSelectionFromListView()
{
	TArray<TSharedRef<FVariantManagerActorNode>> ListViewActorNodes;

	TArray<TSharedRef<FVariantManagerDisplayNode>> ListViewSelectedNodes = GetSelectedItems();
	for (TSharedRef<FVariantManagerDisplayNode>& SelectedNode : ListViewSelectedNodes)
	{
		if (SelectedNode->GetType() == EVariantManagerNodeType::Actor)
		{
			TSharedRef<FVariantManagerActorNode> SelectedActor = StaticCastSharedRef<FVariantManagerActorNode>(SelectedNode);
			ListViewActorNodes.Add(SelectedActor);
		}
	}

	FVariantManagerSelection& Selection = VariantManagerPtr.Pin()->GetSelection();
	Selection.SetActorNodeSelectionTo(ListViewActorNodes, false);
}

void SVariantManagerActorListView::UpdateListViewFromSelection()
{
	FVariantManagerSelection& Selection = VariantManagerPtr.Pin()->GetSelection();
	const TSet<TSharedRef<FVariantManagerActorNode>> SelectedActorNodes = Selection.GetSelectedActorNodes();

	TSet<TSharedRef<FVariantManagerDisplayNode>> SelectedDisplayNodes;
	for (const TSharedRef<FVariantManagerActorNode>& SelectedActorNode : SelectedActorNodes)
	{
		SelectedDisplayNodes.Add(StaticCastSharedRef<FVariantManagerDisplayNode>(SelectedActorNode));
	}

	SelectedItems = SelectedDisplayNodes;
}

TSharedRef<ITableRow> SVariantManagerActorListView::OnGenerateActorRow(TSharedRef<FVariantManagerDisplayNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVariantManagerTableRow, OwnerTable, Item);
}

TSharedPtr<SWidget> SVariantManagerActorListView::OnActorListContextMenuOpening()
{
	TSharedPtr<FVariantManager> VariantManager = VariantManagerPtr.Pin();

	// Add actor options on top if we have them
	const TSet<TSharedRef<FVariantManagerActorNode>> SelectedNodes = VariantManager->GetSelection().GetSelectedActorNodes();
	auto SelectedNodesArray = SelectedNodes.Array();
	if (SelectedNodes.Num() > 0 && SelectedNodesArray[0]->IsSelectable())
	{
		FMenuBuilder MenuBuilder(true, VariantManager->GetVariantManagerWidget()->GetActorListCommandBindings());
		SelectedNodesArray[0]->BuildContextMenu(MenuBuilder);
		return MenuBuilder.MakeWidget();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
