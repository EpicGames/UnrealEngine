// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

class FVariantManagerDisplayNode;


class FVariantManager;


class SVariantManagerActorListView : public SListView<TSharedRef<FVariantManagerDisplayNode>>
{
public:
	SLATE_BEGIN_ARGS(SVariantManagerActorListView) {}
	SLATE_ARGUMENT( const TArray<TSharedRef<FVariantManagerDisplayNode>>* , ListItemsSource)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FVariantManager> InVariantManagerPtr);

	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	// We override all of these because we want to behave like the rest of the engine: Highlight on mouse
	// down, but only select on release/mouse up. Besides, some of these fire one after the other like
	// clear + setItemSelection, which would break things in SVariantManager or cause unecessary updates.
	// The first three fire on mouse down and highlight, the last one fires on mouse up/key down/touch up and selects
	virtual void Private_SetItemSelection( TSharedRef<FVariantManagerDisplayNode> TheItem, bool bShouldBeSelected, bool bWasUserDirected = false ) override;
	virtual void Private_ClearSelection() override;
	virtual void Private_SelectRangeFromCurrentTo(TSharedRef<FVariantManagerDisplayNode> InRangeSelectionEnd) override;
	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override;

	void UpdateSelectionFromListView();
	void UpdateListViewFromSelection();

private:

	TSharedRef<ITableRow> OnGenerateActorRow(TSharedRef<FVariantManagerDisplayNode> Item, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<SWidget> OnActorListContextMenuOpening();

	TWeakPtr<FVariantManager> VariantManagerPtr;
	bool bCanDrop = false;
};
