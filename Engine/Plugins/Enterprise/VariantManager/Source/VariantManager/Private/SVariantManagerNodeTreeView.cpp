// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariantManagerNodeTreeView.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "VariantManager.h"
#include "SVariantManager.h"
#include "SVariantManagerTableRow.h"
#include "VariantManagerNodeTree.h"
#include "VariantManagerSelection.h"
#include "VariantManagerSelection.h"
#include "VariantManager.h"
#include "SVariantManager.h"

#define LOCTEXT_NAMESPACE "SVariantManagerNodeTreeView"


void SVariantManagerNodeTreeView::Construct(const FArguments& InArgs, const TSharedRef<FVariantManagerNodeTree>& InNodeTree)
{
	VariantManagerNodeTree = InNodeTree;

	TSharedPtr<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow).Visibility(EVisibility::Collapsed);

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&RootNodes)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SVariantManagerNodeTreeView::OnGenerateRow)
		.OnGetChildren(this, &SVariantManagerNodeTreeView::OnGetChildren)
		.HeaderRow(HeaderRowWidget)
		.OnExpansionChanged(this, &SVariantManagerNodeTreeView::OnExpansionChanged)
		.AllowOverscroll(EAllowOverscroll::No)
		.OnContextMenuOpening(this, &SVariantManagerNodeTreeView::OnContextMenuOpening)
	);
}

int32 SVariantManagerNodeTreeView::GetDisplayIndexOfNode(FDisplayNodeRef InNode)
{
	return LinearizedItems.Find(InNode);
}

void SVariantManagerNodeTreeView::Refresh()
{
	RootNodes.Reset(VariantManagerNodeTree->GetRootNodes().Num());

	for (const FDisplayNodeRef& RootNode : VariantManagerNodeTree->GetRootNodes())
	{
		if (RootNode->IsExpanded())
		{
			SetItemExpansion(RootNode, true);
		}

		if (!RootNode->IsHidden())
		{
			RootNodes.Add(RootNode);
		}
	}

	RequestTreeRefresh();
}

void SVariantManagerNodeTreeView::SortAsDisplayed(TArray<FDisplayNodeRef>& Nodes)
{
	const TArray<FDisplayNodeRef>& Linearized = this->LinearizedItems;

	TMap<FDisplayNodeRef, int32> ItemIndices;
	for (int32 Index = 0; Index < Linearized.Num(); Index++)
	{
		ItemIndices.Add(Linearized[Index], Index);
	}

	Nodes.Sort([&ItemIndices](const FDisplayNodeRef& A, const FDisplayNodeRef& B)
	{
		return ItemIndices[A] < ItemIndices[B];
	});
}

void SVariantManagerNodeTreeView::Private_SetItemSelection(FDisplayNodeRef TheItem, bool bShouldBeSelected, bool bWasUserDirected)
{
	STreeView::Private_SetItemSelection(TheItem, bShouldBeSelected, bWasUserDirected);
	UpdateSelectionFromTreeView();
}

void SVariantManagerNodeTreeView::Private_ClearSelection()
{
	STreeView::Private_ClearSelection();
	UpdateSelectionFromTreeView();
}

void SVariantManagerNodeTreeView::Private_SelectRangeFromCurrentTo(FDisplayNodeRef InRangeSelectionEnd)
{
	STreeView::Private_SelectRangeFromCurrentTo(InRangeSelectionEnd);
	UpdateSelectionFromTreeView();
}

void SVariantManagerNodeTreeView::Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo)
{
	STreeView::Private_SignalSelectionChanged(SelectInfo);

	FVariantManagerSelection& Selection = VariantManagerNodeTree->GetVariantManager().GetSelection();
	Selection.RequestOutlinerNodeSelectionChangedBroadcast();
}

void SVariantManagerNodeTreeView::UpdateSelectionFromTreeView()
{
	FVariantManagerSelection& Selection = VariantManagerNodeTree->GetVariantManager().GetSelection();
	Selection.SetSelectionTo(GetSelectedItems(), false);
}

void SVariantManagerNodeTreeView::UpdateTreeViewFromSelection()
{
	const TSet<TSharedRef<FVariantManagerDisplayNode>>& SelectedNodes = VariantManagerNodeTree->GetVariantManager().GetSelection().GetSelectedOutlinerNodes();

	FVariantManagerSelection& Selection = VariantManagerNodeTree->GetVariantManager().GetSelection();

	// Do one by one so that the selector item is updated, or else when we duplicate/paste our
	// keyboard selection would return to position zero
	Selection.SuspendBroadcast();
	for (const TSharedRef<FVariantManagerDisplayNode>& Node : SelectedNodes)
	{
		STreeView::Private_SetItemSelection(Node, true, true);
	}
	Selection.ResumeBroadcast();
}

void SVariantManagerNodeTreeView::OnExpansionChanged(FDisplayNodeRef InItem, bool bIsExpanded)
{
	InItem->SetExpansionState(bIsExpanded);
}

TSharedRef<ITableRow> SVariantManagerNodeTreeView::OnGenerateRow(FDisplayNodeRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVariantManagerTableRow, OwnerTable, InDisplayNode);
}

void SVariantManagerNodeTreeView::OnGetChildren(FDisplayNodeRef InParent, TArray<FDisplayNodeRef>& OutChildren) const
{
	for (const auto& Node : InParent->GetChildNodes())
	{
		if (!Node->IsHidden())
		{
			OutChildren.Add(Node);
		}
	}
}

TSharedPtr<SWidget> SVariantManagerNodeTreeView::OnContextMenuOpening()
{
	FVariantManager& VariantManager = VariantManagerNodeTree->GetVariantManager();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, VariantManager.GetVariantManagerWidget()->GetVariantTreeCommandBindings());

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditSectionText", "Edit"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
	MenuBuilder.EndSection();

	const TSet<TSharedRef<FVariantManagerDisplayNode>> SelectedNodes = VariantManager.GetSelection().GetSelectedOutlinerNodes();
	auto SelectedNodesArray = SelectedNodes.Array();
	if (SelectedNodes.Num() > 0 && SelectedNodesArray[0]->IsSelectable())
	{
		SelectedNodesArray[0]->BuildContextMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}



#undef LOCTEXT_NAMESPACE
