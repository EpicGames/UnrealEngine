// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STreeView.h"

class FVariantManagerDisplayNode;
class FVariantManagerNodeTree;

typedef TSharedRef<FVariantManagerDisplayNode> FDisplayNodeRef;


class SVariantManagerNodeTreeView : public STreeView<FDisplayNodeRef>
{
public:

	SLATE_BEGIN_ARGS(SVariantManagerNodeTreeView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FVariantManagerNodeTree>& InNodeTree);

	int32 GetDisplayIndexOfNode(FDisplayNodeRef InNode);

	// Caches the nodes the VariantManagerNodeTree is using and refreshes the display
	void Refresh();

	// Sorts the given nodes so that they are in the same order as they appear on the screen
	// This compares the given nodes with the STreeView's LinearizedItems
	void SortAsDisplayed(TArray<FDisplayNodeRef>& Nodes);

	// We override all of these because we want to behave like the rest of the engine: Highlight on mouse
	// down, but only select on release/mouse up. Besides, some of these fire one after the other like
	// clear + setItemSelection, which would break things in SVariantManager or cause unecessary updates.
	// The first three fire on mouse down and highlight, the last one fires on mouse up/key down/touch up and selects
	virtual void Private_SetItemSelection(FDisplayNodeRef TheItem, bool bShouldBeSelected, bool bWasUserDirected = false) override;
	virtual void Private_ClearSelection() override;
	virtual void Private_SelectRangeFromCurrentTo(FDisplayNodeRef InRangeSelectionEnd) override;
	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override;

	// Sets FVariantManagerSelection to whatever our STreeView has selected
	// Does not broadcast anything
	void UpdateSelectionFromTreeView();
	void UpdateTreeViewFromSelection();

protected:

	void OnExpansionChanged(FDisplayNodeRef InItem, bool bIsExpanded);
	TSharedRef<ITableRow> OnGenerateRow(FDisplayNodeRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetChildren(FDisplayNodeRef InParent, TArray<FDisplayNodeRef>& OutChildren) const;
	TSharedPtr<SWidget> OnContextMenuOpening();

	TSharedPtr<FVariantManagerNodeTree> GetNodeTree()
	{
		return VariantManagerNodeTree;
	}

private:

	TSharedPtr<FVariantManagerNodeTree> VariantManagerNodeTree;
	TArray<FDisplayNodeRef> RootNodes;
};
