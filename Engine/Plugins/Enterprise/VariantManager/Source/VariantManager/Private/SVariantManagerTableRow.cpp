// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariantManagerTableRow.h"

#include "DisplayNodes/VariantManagerActorNode.h"
#include "SVariantManager.h"
#include "VariantManager.h"
#include "VariantManagerDragDropOp.h"
#include "VariantManagerSelection.h"

#include "GameFramework/Actor.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "SVariantManagerTableRow"

/** Construct function for this widget */
void SVariantManagerTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<FVariantManagerDisplayNode>& InNode)
{
	Node = InNode;
	bool bIsSelectable = InNode->IsSelectable();

	STableRow::Construct(
		STableRow::FArguments()
		.Padding(0)
		.Style(InNode->GetRowStyle())
		.OnDragDetected(this, &SVariantManagerTableRow::DragDetected)
		.OnCanAcceptDrop(this, &SVariantManagerTableRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SVariantManagerTableRow::AcceptDrop)
		.OnDragLeave(this, &SVariantManagerTableRow::DragLeave)
		.ShowSelection(bIsSelectable),
		OwnerTableView);

	SetRowContent(InNode->GetCustomOutlinerContent(SharedThis(this)));
}

FReply SVariantManagerTableRow::DragDetected( const FGeometry& InGeometry, const FPointerEvent& InPointerEvent )
{
	if (TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin())
	{
		if (TSharedPtr<FVariantManager> VarMan = PinnedNode->GetVariantManager().Pin())
		{
			const FSlateBrush* Icon = PinnedNode->GetIconBrush();

			FVariantManagerSelection& Selection = VarMan->GetSelection();

			TArray<FDisplayNodeRef> DraggableNodes;
			FText DefaultHoverText;

			// We'll drag a group of nodes based on what type we are (e.g. if we're variant or variant
			// set, drag all of those)
			switch (PinnedNode->GetType())
			{
			case EVariantManagerNodeType::Actor:
			{
				for (const FDisplayNodeRef SelectedNode : Selection.GetSelectedActorNodes())
				{
					if (SelectedNode->CanDrag())
					{
						DraggableNodes.Add(SelectedNode);
					}
				}

				int32 NumNodes = DraggableNodes.Num();
				if (NumNodes > 1)
				{
					Icon = FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetOptionalIcon();
				}

				DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragActorNode", "{0} actor {0}|plural(one=binding,other=bindings)" ), NumNodes);
				break;
			}
			case EVariantManagerNodeType::Function:
			case EVariantManagerNodeType::Property:
			{
				if (PinnedNode.IsValid() && PinnedNode->CanDrag())
				{
					// We only support dragging one node property/function nodes for now
					ensure(DraggableNodes.Num() == 0);

					DraggableNodes.Add(PinnedNode.ToSharedRef());
					DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragPropertyNode", "{0}"), PinnedNode->GetDisplayName());
				}
				break;
			}
			case EVariantManagerNodeType::Variant:  // Intended fallthrough
			case EVariantManagerNodeType::VariantSet:
			{
				for (const FDisplayNodeRef& SelectedNode : Selection.GetSelectedOutlinerNodes())
				{
					if (SelectedNode->CanDrag())
					{
						DraggableNodes.Add(SelectedNode);
					}
				}

				int32 NumNodes = DraggableNodes.Num();

				DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragVariants", "{0} {0}|plural(one=variant,other=variants) and/or variant {0}|plural(one=set,other=sets)" ),
					NumNodes);

				break;
			}
			default:
				break;
			}

			if (DraggableNodes.Num() == 0)
			{
				return FReply::Unhandled();
			}

			if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
			{
				VariantManagerWidget->SortDisplayNodes(DraggableNodes);
			}

			TSharedRef<FVariantManagerDragDropOp> DragDropOp = FVariantManagerDragDropOp::New(DraggableNodes);
			DragDropOp->SetToolTip(DefaultHoverText, Icon);
			DragDropOp->SetupDefaults();

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

void SVariantManagerTableRow::DragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (DecoratedDragDropOp.IsValid())
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
}

TOptional<EItemDropZone> SVariantManagerTableRow::CanAcceptDrop( const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, FDisplayNodeRef DisplayNode )
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		return PinnedNode->CanDrop(DragDropEvent, InItemDropZone);
	}

	return TOptional<EItemDropZone>();
}

FReply SVariantManagerTableRow::AcceptDrop( const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, FDisplayNodeRef DisplayNode )
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		PinnedNode->Drop(DragDropEvent, InItemDropZone);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SVariantManagerTableRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	STableRow::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);

	if (TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin())
	{
		return PinnedNode->OnDoubleClick(InMyGeometry, InMouseEvent);
	}

	return FReply::Unhandled();
}

// Small hack to bypass CanDrop calls to spacer nodes, letting the underlying tree handle the events instead
FReply SVariantManagerTableRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		if (PinnedNode->GetType() == EVariantManagerNodeType::Spacer)
		{
			return FReply::Unhandled();
		}
	}

	return STableRow::OnDragOver(MyGeometry, DragDropEvent);
}

const FName SVariantManagerMultiColumnPropertyTableRow::ColumnPath(TEXT("Path"));
const FName SVariantManagerMultiColumnPropertyTableRow::ColumnProperty(TEXT("Property"));
const FName SVariantManagerMultiColumnPropertyTableRow::ColumnValue(TEXT("Value"));

void SVariantManagerMultiColumnPropertyTableRow::Construct(
	const FArguments& InArgs, 
	const TSharedRef<STableViewBase>& OwnerTableView,
	const TSharedRef<FVariantManagerPropertyNode>& InNode)
{
	Node = InNode;
	bool bIsSelectable = InNode->IsSelectable();

	SMultiColumnTableRow::Construct(
		STableRow::FArguments()
		.Padding(0)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
		.OnDragDetected(this, &SVariantManagerMultiColumnPropertyTableRow::DragDetected)
		.OnCanAcceptDrop(this, &SVariantManagerMultiColumnPropertyTableRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SVariantManagerMultiColumnPropertyTableRow::AcceptDrop)
		.OnDragLeave(this, &SVariantManagerMultiColumnPropertyTableRow::DragLeave)
		.ShowSelection(bIsSelectable),
		OwnerTableView);
}

FReply SVariantManagerMultiColumnPropertyTableRow::DragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin())
	{
		if (TSharedPtr<FVariantManager> VarMan = PinnedNode->GetVariantManager().Pin())
		{
			const FSlateBrush* Icon = PinnedNode->GetIconBrush();

			FVariantManagerSelection& Selection = VarMan->GetSelection();

			TArray<FDisplayNodeRef> DraggableNodes;
			FText DefaultHoverText;

			// We'll drag a group of nodes based on what type we are (e.g. if we're variant or variant
			// set, drag all of those)
			switch (PinnedNode->GetType())
			{
			case EVariantManagerNodeType::Actor:
			{
				for (const FDisplayNodeRef SelectedNode : Selection.GetSelectedActorNodes())
				{
					if (SelectedNode->CanDrag())
					{
						DraggableNodes.Add(SelectedNode);
					}
				}

				int32 NumNodes = DraggableNodes.Num();
				if (NumNodes > 1)
				{
					Icon = FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetOptionalIcon();
				}

				DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragActorNode", "{0} actor {0}|plural(one=binding,other=bindings)" ), NumNodes);
				break;
			}
			case EVariantManagerNodeType::Function:
			case EVariantManagerNodeType::Property:
			{
				if (PinnedNode.IsValid() && PinnedNode->CanDrag())
				{
					// We only support dragging one node property/function nodes for now
					ensure(DraggableNodes.Num() == 0);

					DraggableNodes.Add(PinnedNode.ToSharedRef());
					DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragPropertyNode", "{0}"), PinnedNode->GetDisplayName());
				}
				break;
			}
			case EVariantManagerNodeType::Variant:  // Intended fallthrough
			case EVariantManagerNodeType::VariantSet:
			{
				for (const FDisplayNodeRef& SelectedNode : Selection.GetSelectedOutlinerNodes())
				{
					if (SelectedNode->CanDrag())
					{
						DraggableNodes.Add(SelectedNode);
					}
				}

				int32 NumNodes = DraggableNodes.Num();

				DefaultHoverText = FText::Format(NSLOCTEXT("VariantManagerTableRow", "DragVariants", "{0} {0}|plural(one=variant,other=variants) and/or variant {0}|plural(one=set,other=sets)" ),
					NumNodes);

				break;
			}
			default:
				break;
			}

			if (DraggableNodes.Num() == 0)
			{
				return FReply::Unhandled();
			}

			if (TSharedPtr<SVariantManager> VariantManagerWidget = VarMan->GetVariantManagerWidget())
			{
				VariantManagerWidget->SortDisplayNodes(DraggableNodes);
			}

			TSharedRef<FVariantManagerDragDropOp> DragDropOp = FVariantManagerDragDropOp::New(DraggableNodes);
			DragDropOp->SetToolTip(DefaultHoverText, Icon);
			DragDropOp->SetupDefaults();

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

void SVariantManagerMultiColumnPropertyTableRow::DragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (DecoratedDragDropOp.IsValid())
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
}

TOptional<EItemDropZone> SVariantManagerMultiColumnPropertyTableRow::CanAcceptDrop(
	const FDragDropEvent& DragDropEvent, 
	EItemDropZone InItemDropZone,
	TSharedPtr<FVariantManagerPropertyNode> DisplayNode)
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		return PinnedNode->CanDrop(DragDropEvent, InItemDropZone);
	}

	return TOptional<EItemDropZone>();
}

FReply SVariantManagerMultiColumnPropertyTableRow::AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FVariantManagerPropertyNode> DisplayNode)
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		PinnedNode->Drop(DragDropEvent, InItemDropZone);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SVariantManagerMultiColumnPropertyTableRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	STableRow::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);

	if (TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin())
	{
		return PinnedNode->OnDoubleClick(InMyGeometry, InMouseEvent);
	}

	return FReply::Unhandled();
}

FReply SVariantManagerMultiColumnPropertyTableRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FVariantManagerDisplayNode> PinnedNode = Node.Pin();
	if (PinnedNode.IsValid())
	{
		if (PinnedNode->GetType() == EVariantManagerNodeType::Spacer)
		{
			return FReply::Unhandled();
		}
	}

	return STableRow::OnDragOver(MyGeometry, DragDropEvent);
}

TSharedRef<SWidget> SVariantManagerMultiColumnPropertyTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (TSharedPtr<FVariantManagerPropertyNode> NodePtr = Node.Pin())
	{
		if (InColumnName == ColumnPath)
		{
			return SNew(SBox)
				[
					SNew(SHorizontalBox)
					.Visibility(EVisibility::HitTestInvisible)

					+ SHorizontalBox::Slot()
					.Padding(2.0f, 0.0f, 2.0f, 0.0f)
					.MaxWidth(15.0f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(16, 16))
						.Image(FAppStyle::Get().GetBrush(NodePtr->IsActorProperty() ? "ClassIcon.Actor" : "ClassIcon.ActorComponent"))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 2.0f, 0.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
						.Text(FText::FromString(NodePtr->GetSubPath()))
					]
				];
		}
		else if (InColumnName == ColumnProperty)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
					.Text(FText::FromString(NodePtr->GetPropertyName()))
				];
		}
		else if (InColumnName == ColumnValue)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				[
					NodePtr->GetCustomValueContent()
				];
		}
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
