// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayNodes/VariantManagerDisplayNode.h"
#include "DisplayNodes/VariantManagerPropertyNode.h"
#include "Widgets/Views/SListView.h" // IWYU pragma: keep

class STableViewBase;

typedef TSharedRef<FVariantManagerDisplayNode> FDisplayNodeRef;

/** Represents a row in the VariantManager's tree views and list views */
class SVariantManagerTableRow : public STableRow<FDisplayNodeRef>
{
public:
	SLATE_BEGIN_ARGS(SVariantManagerTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<FVariantManagerDisplayNode>& InNode);

	TSharedPtr<FVariantManagerDisplayNode> GetDisplayNode() const
	{
		return Node.Pin();
	}

	// We subscribe these to base class STableRow's events
	FReply DragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);
	void DragLeave(const FDragDropEvent& DragDropEvent);
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, FDisplayNodeRef DisplayNode);
	FReply AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, FDisplayNodeRef DisplayNode);

	// STableRow interface
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End STableRow interface

private:

	mutable TWeakPtr<FVariantManagerDisplayNode> Node;
};


/** Represents a multi-column row in the VariantManager's list views for displaying properties (and function callers) */
class SVariantManagerMultiColumnPropertyTableRow : public SMultiColumnTableRow<TSharedPtr<FVariantManagerPropertyNode>>
{
public:
	static const FName ColumnPath;
	static const FName ColumnProperty;
	static const FName ColumnValue;

	SLATE_BEGIN_ARGS(SVariantManagerMultiColumnPropertyTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<FVariantManagerPropertyNode>& InNode);

	TSharedPtr<FVariantManagerDisplayNode> GetDisplayNode() const
	{
		return Node.Pin();
	}

	// We subscribe these to base class STableRow's events
	FReply DragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);
	void DragLeave(const FDragDropEvent& DragDropEvent);
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FVariantManagerPropertyNode> DisplayNode);
	FReply AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FVariantManagerPropertyNode> DisplayNode);

	// STableRow interface
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End STableRow interface

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:

	mutable TWeakPtr<FVariantManagerPropertyNode> Node;
};
