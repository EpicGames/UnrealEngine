// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManager.h"

#include "CapturableProperty.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelVariantSets.h"
#include "LevelVariantSetsFunctionDirector.h"
#include "PropertyValueColor.h"
#include "PropertyValueMaterial.h"
#include "PropertyValueOption.h"
#include "PropertyValueSoftObject.h"
#include "SVariantManager.h"
#include "Variant.h"
#include "VariantManagerLog.h"
#include "VariantManagerNodeTree.h"
#include "VariantManagerPropertyCapturer.h"
#include "VariantManagerSelection.h"
#include "VariantObjectBinding.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"

#include "Dialogs/SCaptureDialog.h"
#include "Editor.h"
#include "FunctionCaller.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ObjectTools.h"
#include "UObject/StrongObjectPtr.h"

DEFINE_LOG_CATEGORY(LogVariantManager);

#define LOCTEXT_NAMESPACE "VariantManager"

FVariantManager::FVariantManager()
	: Selection(MakeShared< FVariantManagerSelection >())
	, NodeTree(MakeShared< FVariantManagerNodeTree >(*this))
{
}

FVariantManager::~FVariantManager()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	VariantManagerWidget.Reset();
}

void FVariantManager::Close()
{
	VariantManagerWidget.Reset();
}

void FVariantManager::InitVariantManager(ULevelVariantSets* InLevelVariantSets)
{
	CurrentLevelVariantSets = InLevelVariantSets;

	// When undo occurs, get a notification so we can make sure our view is up to date
	GEditor->RegisterForUndo(this);
}

UBlueprint* FVariantManager::GetOrCreateDirectorBlueprint(ULevelVariantSets* InLevelVariantSets)
{
	// Create a new director. This is a custom blueprint class deriving from ULevelVariantSetsFunctionDirector
	// that will receive our function. We're storing this class also, so that we can instantiate this subclass
	// later when we want to execute these functions

	UBlueprint* Director = Cast<UBlueprint>(InLevelVariantSets->GetDirectorGeneratedBlueprint());
	if (!Director || !IsValidChecked(Director) || Director->IsUnreachable())
	{
		FName BlueprintName = "LevelVariantSetsDirector";
		Director = FKismetEditorUtilities::CreateBlueprint(ULevelVariantSetsFunctionDirector::StaticClass(), InLevelVariantSets, BlueprintName, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());

		InLevelVariantSets->SetDirectorGeneratedBlueprint(Director);
	}

	return Director;
}

UK2Node_FunctionEntry* FVariantManager::CreateDirectorFunction(ULevelVariantSets* InLevelVariantSets, FName InFunctionName, UClass* PinClassType, bool bCreateWithArguments)
{
	UBlueprint* Director = GetOrCreateDirectorBlueprint(InLevelVariantSets);

	static FString DefaultEventName = "Function ";
	FName UniqueGraphName = FBlueprintEditorUtils::FindUniqueKismetName(Director, InFunctionName.IsNone() ? DefaultEventName : InFunctionName.ToString());

	Director->Modify();

	UEdGraph* Graph = FBlueprintEditorUtils::CreateNewGraph(Director, UniqueGraphName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

	constexpr bool bIsUserCreated = false;
	FBlueprintEditorUtils::AddFunctionGraph<UClass>(Director, Graph, bIsUserCreated, nullptr);

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	Graph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);

	if (ensure(EntryNodes.Num() == 1 && EntryNodes[0]))
	{
		UK2Node_FunctionEntry* EntryNode = EntryNodes[0];

		int32 ExtraFunctionFlags = ( FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public );
		EntryNode->AddExtraFlags(ExtraFunctionFlags);

		EntryNode->bIsEditable = true;
		EntryNode->MetaData.Category = LOCTEXT("DefaultCategory", "Director functions");
		EntryNode->MetaData.bCallInEditor = false;

		EntryNode->NodeComment = TEXT("Will be called by the Variant Manager when switching variants.\nThe target pin will receive a reference to the actor to which\nthis function caller is bound.\n\nYou can modify this target pin, either removing it\nor changing its type to another object reference type.\nIf you do, however, in order for a reference to the bound actor to be assigned\nto the pin, the actor must be of a type derived from the pin type.");
		EntryNode->bCommentBubblePinned = true;
		EntryNode->bCommentBubbleVisible = true;

		FEdGraphPinType PinType;
		PinType.PinCategory = PinClassType && PinClassType->IsChildOf(UInterface::StaticClass()) ? UEdGraphSchema_K2::PC_Interface : UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = PinClassType ? PinClassType : UObject::StaticClass();
		EntryNode->CreateUserDefinedPin(TARGET_PIN_NAME, PinType, EGPD_Output, true);

		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = ULevelVariantSets::StaticClass();
		EntryNode->CreateUserDefinedPin(LEVEL_VARIANT_SETS_PIN_NAME, PinType, EGPD_Output, true);

		PinType.PinSubCategoryObject = UVariantSet::StaticClass();
		EntryNode->CreateUserDefinedPin(VARIANT_SET_PIN_NAME, PinType, EGPD_Output, true);

		PinType.PinSubCategoryObject = UVariant::StaticClass();
		EntryNode->CreateUserDefinedPin(VARIANT_PIN_NAME, PinType, EGPD_Output, true);

		if (bCreateWithArguments)
		{
			FEdGraphPinType ArgsPinType;
			ArgsPinType.PinCategory = UEdGraphSchema_K2::PC_Name;
			ArgsPinType.ContainerType = EPinContainerType::Map;
			ArgsPinType.PinSubCategory = UEdGraphSchema_K2::PC_Name;
			FEdGraphPinType ValuePinType;
			ValuePinType.PinCategory = UEdGraphSchema_K2::PC_String;
			ArgsPinType.PinValueType = FEdGraphTerminalType::FromPinType(ValuePinType);
			EntryNode->CreateUserDefinedPin(ARGUMENTS_PIN_NAME, ArgsPinType, EGPD_Output, true);
		}

		EntryNode->ReconstructNode();

		return EntryNode;
	}

	return nullptr;
}

UK2Node_FunctionEntry* FVariantManager::CreateDirectorFunctionFromFunction(UFunction* QuickBindFunction, UClass* PinClassType, bool bCreateWithArguments)
{
	FString DesiredNewEventName = FString(TEXT("Call ")) + QuickBindFunction->GetName();

	// Create a single event binding and point all events in this property handle to it
	UK2Node_FunctionEntry* NewFunctionEntry = CreateDirectorFunction(GetCurrentLevelVariantSets(), FName(*DesiredNewEventName), PinClassType, bCreateWithArguments);
	if (!ensure(NewFunctionEntry))
	{
		return nullptr;
	}

	NewFunctionEntry->bCommentBubblePinned = false;
	NewFunctionEntry->bCommentBubbleVisible = false;

	UEdGraph* Graph = NewFunctionEntry->GetGraph();

	// Make a call function template
	UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_CallFunction>(Graph);
	CallFuncNode->NodePosX = NewFunctionEntry->NodePosX + NewFunctionEntry->NodeWidth + 200;
	CallFuncNode->NodePosY = NewFunctionEntry->NodePosY;
	CallFuncNode->CreateNewGuid();
	CallFuncNode->SetFromFunction(QuickBindFunction);
	CallFuncNode->PostPlacedNewNode();
	CallFuncNode->ReconstructNode();

	Graph->AddNode(CallFuncNode, false, false);

	// Connect the exec pins together
	{
		UEdGraphPin* ThenPin = NewFunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then);
		UEdGraphPin* ExecPin = CallFuncNode->GetExecPin();

		if (ThenPin && ExecPin)
		{
			const UEdGraphSchema* Schema = ThenPin->GetSchema();
			Schema->TryCreateConnection(ThenPin, ExecPin);
		}
	}

	// Connect the object target pin to the self (input) pin on the call function node
	if (UEdGraphPin* SelfPin = CallFuncNode->FindPin(UEdGraphSchema_K2::PSC_Self))
	{
		for (UEdGraphPin* Pin : NewFunctionEntry->Pins)
		{
			if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface  || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
			{
				const UEdGraphSchema* Schema = Pin->GetSchema();
				Schema->TryCreateConnection(Pin, SelfPin);
			}
		}
	}

	return NewFunctionEntry;
}

void FVariantManager::CreateFunctionCaller(const TArray<UVariantObjectBinding*>& Bindings)
{
	// Call this here as we use the same function to add an empty caller as a valid one, and
	// we need the director class setup in order to try and add a reference to one of its functions
	GetOrCreateDirectorBlueprint(GetCurrentLevelVariantSets());

	TArray<FFunctionCaller> NoneFuncCaller = {FFunctionCaller()};

	for (UVariantObjectBinding* Binding : Bindings)
	{
		AddFunctionCallers(NoneFuncCaller, Binding);
	}
}

void FVariantManager::AddPropertyCaptures(const TArray<UPropertyValue*>& OfTheseProperties, UVariantObjectBinding* ToThisBinding)
{
	if (OfTheseProperties.Num() < 1)
	{
		return;
	}

	ToThisBinding->AddCapturedProperties(OfTheseProperties);
}

void FVariantManager::AddObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant, int32 InsertionIndex, bool bReplaceOld)
{
	if (TheseBindings.Num() < 1)
	{
		return;
	}

	TArray<UVariantObjectBinding*> BindingsToAdd = TheseBindings;

	// Discard bindings that the variant already has
	// AddBindings will always replace so that it can be used to reorder, meaning we have to take care of this ourselves
	if (!bReplaceOld)
	{
		TSet<FString> ExistingBindingGuids;
		for (const UVariantObjectBinding* Binding : ToThisVariant->GetBindings())
		{
			ExistingBindingGuids.Add(Binding->GetObjectPath());
		}

		for (int32 Index = BindingsToAdd.Num() - 1; Index >= 0; Index--)
		{
			if (ExistingBindingGuids.Contains(BindingsToAdd[Index]->GetObjectPath()))
			{
				BindingsToAdd.RemoveAt(Index);
			}
		}
	}

	ToThisVariant->AddBindings(BindingsToAdd, InsertionIndex);
}

void FVariantManager::AddVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex)
{
	if (TheseVariants.Num() < 1)
	{
		return;
	}

	ToThisVariantSet->AddVariants(TheseVariants, InsertionIndex);
}

void FVariantManager::AddVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVarSets, int32 InsertionIndex)
{
	if (TheseVariantSets.Num() < 1)
	{
		return;
	}

	ToThisLevelVarSets->AddVariantSets(TheseVariantSets, InsertionIndex);
}

void FVariantManager::AddFunctionCallers(const TArray<FFunctionCaller>& Functions, UVariantObjectBinding* Binding)
{
	Binding->AddFunctionCallers(Functions);
}

void FVariantManager::RemovePropertyCapturesFromParent(const TArray<UPropertyValue*>& TheseProps)
{
	for (UPropertyValue* Prop : TheseProps)
	{
		Prop->GetParent()->RemoveCapturedProperties({Prop});
	}
}

void FVariantManager::RemoveObjectBindingsFromParent(const TArray<UVariantObjectBinding*>& TheseBindings)
{
	for (UVariantObjectBinding* Binding : TheseBindings)
	{
		Binding->GetParent()->RemoveBindings({Binding});
	}
}

void FVariantManager::RemoveVariantsFromParent(const TArray<UVariant*>& TheseVariants)
{
	for (UVariant* Variant : TheseVariants)
	{
		// Don't delete thumbnails as these are not cached as transactions, meaning if we
		// undo after removing this variant we won't recover our thumbnail
		//ThumbnailTools::CacheThumbnail(Variant->GetFullName(), nullptr, Variant->GetOutermost());
		Variant->GetParent()->RemoveVariants({Variant});
	}
}

void FVariantManager::RemoveVariantSetsFromParent(const TArray<UVariantSet*>& TheseVariantSets)
{
	for (UVariantSet* VariantSet : TheseVariantSets)
	{
		if (ULevelVariantSets* Parent = VariantSet->GetParent())
		{
			Parent->RemoveVariantSets({VariantSet});
		}
	}
}

void FVariantManager::RemoveFunctionCallers(const TArray<FFunctionCaller*>& TheseCallers, UVariantObjectBinding* FromThisBinding)
{
	FromThisBinding->RemoveFunctionCallers(TheseCallers);
}

void FVariantManager::MoveObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant)
{
	if (TheseBindings.Num() < 1)
	{
		return;
	}

	// Build auxiliary datastructure to help check for movable bindings
	TSet<UObject*> BoundObjects;
	for (UVariantObjectBinding* Binding : ToThisVariant->GetBindings())
	{
		BoundObjects.Add(Binding->GetObject());
	}

	// See which bindings can actually be moved
	TArray<UVariantObjectBinding*> BindingsWeCanMove;
	for (UVariantObjectBinding* Binding : TheseBindings)
	{
		if (!BoundObjects.Contains(Binding->GetObject()))
		{
			BindingsWeCanMove.Add(Binding);
		}
	}

	// Get strong pointers to the bindings we'll move just in case
	TArray<TStrongObjectPtr<UVariantObjectBinding>> PinArray;
	for (UVariantObjectBinding* RawPtr : BindingsWeCanMove)
	{
		PinArray.Add(TStrongObjectPtr<UVariantObjectBinding>(RawPtr));
	}

	RemoveObjectBindingsFromParent(BindingsWeCanMove);
	AddObjectBindings(BindingsWeCanMove, ToThisVariant);
}

void FVariantManager::MoveVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex)
{
	if (TheseVariants.Num() < 1)
	{
		return;
	}

	// Get strong pointers to the variants we'll move just in case
	TArray<TStrongObjectPtr<UVariant>> PinArray;
	for (UVariant* RawPtr : TheseVariants)
	{
		PinArray.Add(TStrongObjectPtr<UVariant>(RawPtr));
	}

	// If these variants came from other variant sets, we'll need to move the thumbnails
	// so we lets just do it for all of them
	TArray<FObjectThumbnail*> Thumbnails = GetVariantThumbnails(TheseVariants);
	AddVariants(TheseVariants, ToThisVariantSet, InsertionIndex);
	SetVariantThumbnails(Thumbnails, TheseVariants);
}

void FVariantManager::MoveVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVariantSets, int32 InsertionIndex)
{
	if (TheseVariantSets.Num() < 1)
	{
		return;
	}

	// Get strong pointers to the variant sets we'll move just in case
	TArray<TStrongObjectPtr<UVariantSet>> PinArray;
	for (UVariantSet* RawPtr : TheseVariantSets)
	{
		PinArray.Add(TStrongObjectPtr<UVariantSet>(RawPtr));
	}

	TMap<const UVariantSet*, TArray<FObjectThumbnail*>> VariantThumbnails;
	for (const UVariantSet* VarSet : TheseVariantSets)
	{
		VariantThumbnails.Add(VarSet, GetVariantThumbnails(VarSet->GetVariants()));
	}

	AddVariantSets(TheseVariantSets, ToThisLevelVariantSets, InsertionIndex);

	for (const auto& Pair: VariantThumbnails)
	{
		const UVariantSet* VarSet = Pair.Key;
		const TArray<FObjectThumbnail*>& Thumbs = Pair.Value;

		SetVariantThumbnails(Thumbs, VarSet->GetVariants());
	}
}

void FVariantManager::DuplicateObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant, int32 InsertionIndex)
{
	if (TheseBindings.Num() < 1)
	{
		return;
	}

	TArray<UVariantObjectBinding*> DuplicatedBindings;
	for (UVariantObjectBinding* RawPtr : TheseBindings)
	{
		DuplicatedBindings.Add(DuplicateObject(RawPtr, nullptr));
	}

	AddObjectBindings(DuplicatedBindings, ToThisVariant, InsertionIndex);
}

void FVariantManager::DuplicateVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex)
{
	if (TheseVariants.Num() < 1)
	{
		return;
	}

	TArray<UVariant*> DuplicatedVariants;
	for (UVariant* RawPtr : TheseVariants)
	{
		UVariant* NewVar = DuplicateObject(RawPtr, nullptr);
		NewVar->SetDisplayText(RawPtr->GetDisplayText());
		DuplicatedVariants.Add(NewVar);
	}

	AddVariants(DuplicatedVariants, ToThisVariantSet, InsertionIndex);

	CopyVariantThumbnails(DuplicatedVariants, TheseVariants);
}

void FVariantManager::DuplicateVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVarSets, int32 InsertionIndex)
{
	if (TheseVariantSets.Num() < 1)
	{
		return;
	}

	TArray<UVariantSet*> DuplicatedVariantSets;
	for (UVariantSet* RawPtr : TheseVariantSets)
	{
		UVariantSet* NewVarSet = DuplicateObject(RawPtr, nullptr);
		NewVarSet->SetDisplayText(RawPtr->GetDisplayText());
		DuplicatedVariantSets.Add(NewVarSet);
	}

	AddVariantSets(DuplicatedVariantSets, GetCurrentLevelVariantSets(), InsertionIndex);

	for (int32 VarSetIndex = 0; VarSetIndex < DuplicatedVariantSets.Num(); VarSetIndex++)
	{
		CopyVariantThumbnails(DuplicatedVariantSets[VarSetIndex]->GetVariants(), TheseVariantSets[VarSetIndex]->GetVariants());
	}
}

TArray<UPropertyValue*> FVariantManager::CreatePropertyCaptures(const TArray<TSharedPtr<FCapturableProperty>>& OfTheseProperties, const TArray<UVariantObjectBinding*>& InTheseBindings, bool bSilenceWarnings)
{
	if (OfTheseProperties.Num() < 1)
	{
		return TArray<UPropertyValue*>();
	}

	TArray<UPropertyValue*> AllNewPropVals;

	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		TSet<FString> ExistingPropertyNames;
		const TArray<UPropertyValue*>& ExistingProps = Binding->GetCapturedProperties();
		for (UPropertyValue* ExistingProp : ExistingProps)
		{
			ExistingPropertyNames.Add(ExistingProp->GetFullDisplayString());
		}

		TArray<UPropertyValue*> NewPropVals;

		// Create a UPropertyValue for each FCapturableProperty
		for (const TSharedPtr<FCapturableProperty>& PropAndDisplay : OfTheseProperties)
		{
			if (ExistingPropertyNames.Contains(PropAndDisplay->DisplayName))
			{
				if (!bSilenceWarnings)
				{
					UE_LOGF(LogVariantManager, Log, "Ignoring attempt to capture property '%ls' as it is already captured for actor binding '%ls'", *PropAndDisplay->DisplayName, *Binding->GetObject()->GetName());
				}
				continue;
			}

			UPropertyValue* NewPropVal = nullptr;
			FFieldClass* PropClass = nullptr;

			switch (PropAndDisplay->CaptureType)
			{
			case EPropertyValueCategory::Material:
			{
				NewPropVal = NewObject<UPropertyValueMaterial>(GetTransientPackage(), UPropertyValueMaterial::StaticClass(), NAME_None, RF_Public|RF_Transactional);
				PropClass = FObjectProperty::StaticClass();
				break;
			}
			case EPropertyValueCategory::Color:
			{
				NewPropVal = NewObject<UPropertyValueColor>(GetTransientPackage(), UPropertyValueColor::StaticClass(), NAME_None, RF_Public|RF_Transactional);
				PropClass = FStructProperty::StaticClass();
				break;
			}
			case EPropertyValueCategory::Option:
			{
				NewPropVal = NewObject<UPropertyValueOption>(GetTransientPackage(), UPropertyValueOption::StaticClass(), NAME_None, RF_Public|RF_Transactional);
				PropClass = FIntProperty::StaticClass();
				break;
			}
			default: // Generic
			{
				const FProperty* LeafProp = PropAndDisplay->Prop.GetLeafMostProperty().Property.Get();

				if (LeafProp && LeafProp->GetClass()->IsChildOf(FSoftObjectProperty::StaticClass()))
				{
					NewPropVal = NewObject<UPropertyValueSoftObject>(GetTransientPackage(), UPropertyValueSoftObject::StaticClass(), NAME_None, RF_Public|RF_Transactional);
					PropClass = FSoftObjectProperty::StaticClass();
				}
				else if (LeafProp)
				{
					NewPropVal = NewObject<UPropertyValue>(GetTransientPackage(), UPropertyValue::StaticClass(), NAME_None, RF_Public|RF_Transactional);
					PropClass = LeafProp->GetClass();
				}
				else
				{
					UObject* BoundObject = Binding->GetObject();
					FString ObjectName = BoundObject ? BoundObject->GetName() : TEXT("<invalid actor>");

					UE_LOGF(LogVariantManager, Error, "Failed to capture property with path '%ls' for actor '%ls'", *PropAndDisplay->DisplayName, *ObjectName);
				}
				break;
			}
			}

			if (!NewPropVal || !PropClass)
			{
				continue;
			}

			NewPropVal->Init(PropAndDisplay->ToCapturedPropSegmentArray(), PropClass, PropAndDisplay->DisplayName, PropAndDisplay->PropertySetterName, PropAndDisplay->CaptureType);

			UObject* BoundObject = Binding->GetObject();
			FString ObjectName = BoundObject->GetName();
			if (AActor* BoundActor = Cast<AActor>(BoundObject))
			{
				ObjectName = BoundActor->GetActorLabel();
			}

			if (NewPropVal->Resolve(Binding->GetObject()))
			{
				NewPropVals.Add(NewPropVal);
			}
			else if (!bSilenceWarnings)
			{
				UE_LOGF(LogVariantManager, Log, "Actor '%ls' does not have a property with path '%ls', so it will not be captured for this actor", *ObjectName, *NewPropVal->GetFullDisplayString());
			}
		}

		AllNewPropVals.Append(NewPropVals);
		AddPropertyCaptures(NewPropVals, Binding);
	}

	// Initialize recorded data after adding so that we have a valid Outer. Also need to do
	// this immediately after capturing or else it will only be done when opening the GUI the first time, which
	// doesn't work well for the python API
	for (UPropertyValue* PropVal : AllNewPropVals)
	{
		PropVal->RecordDataFromResolvedObject();
	}

	return AllNewPropVals;
}

void FVariantManager::MergeObjectBindings(const UVariantObjectBinding* ThisBinding, UVariantObjectBinding* IntoThisBinding)
{
	const TArray<UPropertyValue*>& PropsToAdd = ThisBinding->GetCapturedProperties();

	AddPropertyCaptures(PropsToAdd, IntoThisBinding);
}

void FVariantManager::MergeVariants(const UVariant* ThisVar, UVariant* IntoThisVar)
{
	const TArray<UVariantObjectBinding*>& SrcBindings = ThisVar->GetBindings();
	const TArray<UVariantObjectBinding*>& DstBindings = IntoThisVar->GetBindings();

	// Record which objects we have before adding (those bindings will need to be merged)
	TMap<UObject*, UVariantObjectBinding*> DstBoundObjects;
	for (UVariantObjectBinding* Binding : DstBindings)
	{
		UObject* BindingsObject = Binding->GetObject();
		if (!BindingsObject)
		{
			continue;
		}

		DstBoundObjects.Add(BindingsObject, Binding);
	}

	// Merge existent bindings
	for (UVariantObjectBinding* Binding : SrcBindings)
	{
		UObject* BindingsObject = Binding->GetObject();
		UVariantObjectBinding** TargetBindingPtr = DstBoundObjects.Find(BindingsObject);

		// Target already has this binding, so we should merge them
		if (TargetBindingPtr)
		{
			MergeObjectBindings(Binding, *TargetBindingPtr);
		}
	}

	// Add all src bindings. Only the non-existent ones will be added
	DuplicateObjectBindings(SrcBindings, IntoThisVar);
}

TArray<UPropertyValue*> FVariantManager::CreateTransformPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		// Workaround how different types of actors may have different root component names. If we did all
		// at once and contained a binding to a StaticMeshActor as well as a binding to an Actor, scanning
		// the actor we'd find "RootComponent" and scanning the StaticMeshActor we'd find "StaticMeshComponent".
		// They're both the root, but these two properties would resolve on the StaticMeshActor, which is not
		// what we want (just want one property for each)
		TArray<TSharedPtr<FCapturableProperty>> TransformProps;
		FVariantManagerPropertyCapturer::CaptureTransform({Binding->GetObject()}, TransformProps);

		// Silence the warnings because it's pretty common to capture a static mesh and a simple actor
		// Since their scene roots have different names/paths, this function would emit a log message letting
		// the user know it won't capture the static mesh's property on actors and vice-versa
		Result.Append(CreatePropertyCaptures(TransformProps, {Binding}, true));
	}
	return Result;
}

TArray<UPropertyValue*> FVariantManager::CreateLocationPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		TArray<TSharedPtr<FCapturableProperty>> LocationProps;
		FVariantManagerPropertyCapturer::CaptureLocation({Binding->GetObject()}, LocationProps);
		Result.Append(CreatePropertyCaptures(LocationProps, {Binding}, true));
	}
	return Result;
}

TArray<UPropertyValue*> FVariantManager::CreateRotationPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		TArray<TSharedPtr<FCapturableProperty>> RotationProps;
		FVariantManagerPropertyCapturer::CaptureRotation({Binding->GetObject()}, RotationProps);
		Result.Append(CreatePropertyCaptures(RotationProps, {Binding}, true));
	}
	return Result;
}

TArray<UPropertyValue*> FVariantManager::CreateScale3DPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		TArray<TSharedPtr<FCapturableProperty>> ScaleProps;
		FVariantManagerPropertyCapturer::CaptureScale3D({Binding->GetObject()}, ScaleProps);
		Result.Append(CreatePropertyCaptures(ScaleProps, {Binding}, true));
	}
	return Result;
}

TArray<UPropertyValue*> FVariantManager::CreateVisibilityPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		// Workaround how different types of actors may have different root component names. If we did all
		// at once and contained a binding to a StaticMeshActor as well as a binding to an Actor, scanning
		// the actor we'd find "RootComponent" and scanning the StaticMeshActor we'd find "StaticMeshComponent".
		// They're both the root, but these two properties would resolve on the StaticMeshActor, which is not
		// what we want (just want one property for each)
		TArray<TSharedPtr<FCapturableProperty>> VisibilityProps;
		FVariantManagerPropertyCapturer::CaptureVisibility({Binding->GetObject()}, VisibilityProps);

		// Silence the warnings because it's pretty common to capture a static mesh and a simple actor
		// Since their scene roots have different names/paths, this function would emit a log message letting
		// the user know it won't capture the static mesh's property on actors and vice-versa
		Result.Append(CreatePropertyCaptures(VisibilityProps, {Binding}, true));
	}
	return Result;
}

TArray<UPropertyValue*> FVariantManager::CreateMaterialPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings)
{
	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : InTheseBindings)
	{
		// Workaround how different types of actors may have different root component names. If we did all
		// at once and contained a binding to a StaticMeshActor as well as a binding to an Actor, scanning
		// the actor we'd find "RootComponent" and scanning the StaticMeshActor we'd find "StaticMeshComponent".
		// They're both the root, but these two properties would resolve on the StaticMeshActor, which is not
		// what we want (just want one property for each)
		TArray<TSharedPtr<FCapturableProperty>> MatProps;
		FVariantManagerPropertyCapturer::CaptureMaterial({Binding->GetObject()}, MatProps);

		// Silence the warnings because it's pretty common to capture a static mesh and a simple actor
		// Since their scene roots have different names/paths, this function would emit a log message letting
		// the user know it won't capture the static mesh's property on actors and vice-versa
		Result.Append(CreatePropertyCaptures(MatProps, {Binding}, true));
	}
	return Result;
}

TArray<UVariantObjectBinding*> FVariantManager::CreateObjectBindings(const TArray<AActor*>& OfTheseActors, const TArray<UVariant*>& InTheseVariants, int32 InsertionIndex)
{
	if (OfTheseActors.Num() < 1)
	{
		return TArray<UVariantObjectBinding*>();
	}

	TArray<UVariantObjectBinding*> AllNewBindings;

	// Adds a new binding to each selected actor in every variant of InTheseVariants
	for (UVariant* Variant : InTheseVariants)
	{
		TSet<UObject*> OfTheseObjects;
		for (AActor* Actor : OfTheseActors)
		{
			OfTheseObjects.Add(Cast<UObject>(Actor));
		}

		TArray<UObject*> UnboundActors;
		CanAddActorsToVariant(OfTheseObjects.Array(), Variant, UnboundActors);

		TArray<UVariantObjectBinding*> ExistingBindings = Variant->GetBindings();
		TArray<UVariantObjectBinding*> VarBindingsToTheseActors;  // 1-to-1 correspondence with OfTheseActors
		TArray<UVariantObjectBinding*> NewBindings;  // Bindings we'll create

		for (UObject* SelectedActor : OfTheseObjects)
		{
			if (UnboundActors.Contains(SelectedActor))
			{
				UVariantObjectBinding* NewBinding = NewObject<UVariantObjectBinding>(GetTransientPackage(), NAME_None, RF_Public|RF_Transactional);
				NewBinding->SetObject(SelectedActor);

				VarBindingsToTheseActors.Add(NewBinding);
				NewBindings.Add(NewBinding);
			}
			else
			{
				// Try getting the existing binding that has that actor already
				UVariantObjectBinding** BindingPtr = ExistingBindings.FindByPredicate([SelectedActor](const UVariantObjectBinding* Binding)
				{
					return SelectedActor == Binding->GetObject();
				});

				if (BindingPtr != nullptr)
				{
					VarBindingsToTheseActors.Add(*BindingPtr);
				}
				else
				{
					UE_LOGF(LogVariantManager, Error, "When creating object bindings, variant '%ls' seems to have a binding, but we can't retrieve it!", *SelectedActor->GetName());
				}
			}
		}

		AddObjectBindings(NewBindings, Variant, InsertionIndex);
		AllNewBindings.Append(VarBindingsToTheseActors);
	}

	return AllNewBindings;
}

TArray<UVariantObjectBinding*> FVariantManager::CreateObjectBindingsAndCaptures(const TArray<AActor*>& OfTheseActors, const TArray<UVariant*>& InTheseVariants, int32 InsertionIndex)
{
	if (OfTheseActors.Num() < 1)
	{
		return TArray<UVariantObjectBinding*>();
	}

	TSet<UObject*> AllActors;
	for (AActor* Actor : OfTheseActors)
	{
		AllActors.Add(Cast<UObject>(Actor));
	}

	// Spawn a modal window letting the user pick which of selected actors and properties to capture
	TSharedPtr<SCaptureDialog> CaptureDialog = SCaptureDialog::OpenCaptureDialogAsModalWindow(ECaptureDialogType::ActorAndProperty, AllActors.Array());
	TArray<TSharedPtr<FCapturableProperty>> SelectedProperties = CaptureDialog->GetCurrentCheckedProperties();
	TArray<UObject*> SelectedActors = CaptureDialog->GetCurrentCheckedActors();
	if (!CaptureDialog->GetUserAccepted() || SelectedActors.Num() == 0)
	{
		return TArray<UVariantObjectBinding*>();
	}

	TArray<UVariantObjectBinding*> AllNewBindings; // All created bindings
	TArray<UVariantObjectBinding*> VarBindingsToTheseActors;  // All created or found bindings to OfTheseActors

	// Adds a new binding to each selected actor in every variant of InTheseVariants
	for (UVariant* Variant : InTheseVariants)
	{
		TArray<UObject*> UnboundActors;
		CanAddActorsToVariant(SelectedActors, Variant, UnboundActors);

		TArray<UVariantObjectBinding*> ExistingBindings = Variant->GetBindings();
		TArray<UVariantObjectBinding*> NewBindings;  // Bindings we'll create for this variant

		for (UObject* SelectedActor : SelectedActors)
		{
			if (UnboundActors.Contains(SelectedActor))
			{
				UVariantObjectBinding* NewBinding = NewObject<UVariantObjectBinding>(GetTransientPackage(), NAME_None, RF_Public|RF_Transactional);
				NewBinding->SetObject(SelectedActor);

				VarBindingsToTheseActors.Add(NewBinding);
				NewBindings.Add(NewBinding);
			}
			else
			{
				// Try getting the existing binding that has that actor already
				UVariantObjectBinding** BindingPtr = ExistingBindings.FindByPredicate([SelectedActor](const UVariantObjectBinding* Binding)
				{
					return SelectedActor == Binding->GetObject();
				});

				if (BindingPtr != nullptr)
				{
					VarBindingsToTheseActors.Add(*BindingPtr);
				}
				else
				{
					UE_LOGF(LogVariantManager, Error, "When creating object bindings, variant '%ls' seems to have a binding, but we can't retrieve it!", *SelectedActor->GetName());
				}
			}
		}

		AddObjectBindings(NewBindings, Variant, InsertionIndex);
		AllNewBindings.Append(NewBindings);
	}

	CreatePropertyCaptures(SelectedProperties, VarBindingsToTheseActors);

	return AllNewBindings;
}

UVariant* FVariantManager::CreateVariant(UVariantSet* ToThisSet)
{
	if (!ToThisSet)
	{
		return nullptr;
	}

	UVariant* NewVariant = NewObject<UVariant>(GetTransientPackage(), UVariant::StaticClass(), NAME_None, RF_Public|RF_Transactional);

	AddVariants({NewVariant}, ToThisSet);

	return NewVariant;
}

UVariantSet* FVariantManager::CreateVariantSet(ULevelVariantSets* InThisLevelVariantSets)
{
	if (!InThisLevelVariantSets)
	{
		return nullptr;
	}

	UVariantSet* NewVariantSet = NewObject<UVariantSet>(GetTransientPackage(), UVariantSet::StaticClass(), NAME_None, RF_Public|RF_Transactional);

	AddVariantSets({NewVariantSet}, InThisLevelVariantSets);

	return NewVariantSet;
}

// Returns a thumbnail or nullptr for each variant in SrcArray
TArray<FObjectThumbnail*> FVariantManager::GetVariantThumbnails(const TArray<UVariant*> SrcArray)
{
	TArray<FObjectThumbnail*> Result;
	Result.Reserve(SrcArray.Num());

	for (UVariant* Src : SrcArray)
	{
		FName ObjectFullName = FName(*Src->GetFullName());
		FObjectThumbnail* SrcThumbnail = ThumbnailTools::GetThumbnailForObject(Src);

		// If we don't yet have a thumbnail map, load one from disk if possible
		if (SrcThumbnail == nullptr)
		{
			FThumbnailMap LoadedThumbnails;
			if (ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ObjectFullName}, LoadedThumbnails))
			{
				SrcThumbnail = LoadedThumbnails.Find(ObjectFullName);

				// If we found one, cache it so that we don't have to do this again
				if (SrcThumbnail != nullptr)
				{
					SrcThumbnail = ThumbnailTools::CacheThumbnail(ObjectFullName.ToString(), SrcThumbnail, Src->GetOutermost());
				}
			}
		}

		Result.Add(SrcThumbnail);
	}

	return Result;
}

void FVariantManager::SetVariantThumbnails(const TArray<FObjectThumbnail*> Thumbnails, const TArray<UVariant*> Variants)
{
	for (int32 Index = 0; Index < Variants.Num(); Index++)
	{
		const UVariant* Dst = Variants[Index];
		FObjectThumbnail* SrcThumbnail = Thumbnails[Index];

		FAssetData CurrentAsset(Dst);

		const FString PackageName = CurrentAsset.PackageName.ToString();

		UPackage* AssetPackage = FindObject<UPackage>( nullptr, *PackageName );
		if ( ensure(AssetPackage) )
		{
			FObjectThumbnail* NewThumbnail = ThumbnailTools::CacheThumbnail(Dst->GetFullName(), SrcThumbnail, Dst->GetOutermost());

			// This may now be nullptr as we call this function to also clear thumbnails (i.e. with nullptr Thumbnails). In that
			// case, CacheThumbnail also returns nullptr
			if (NewThumbnail)
			{
				//we need to indicate that the package needs to be resaved
				AssetPackage->MarkPackageDirty();

				// Let the content browser know that we've changed the thumbnail
				NewThumbnail->MarkAsDirty();

				// Signal that the asset was changed if it is loaded so thumbnail pools will update
				if ( CurrentAsset.IsAssetLoaded() )
				{
					CurrentAsset.GetAsset()->PostEditChange();
				}

				//Set that thumbnail as a valid custom thumbnail so it'll be saved out
				NewThumbnail->SetCreatedAfterCustomThumbsEnabled();
			}
		}
	}
}

void FVariantManager::CopyVariantThumbnails(const TArray<UVariant*> DstArray, const TArray<UVariant*> SrcArray)
{
	ensure(DstArray.Num() == SrcArray.Num());
	if (DstArray.Num() < 1)
	{
		return;
	}

	TArray<FObjectThumbnail*> SrcThumbnails = GetVariantThumbnails(SrcArray);

	SetVariantThumbnails(SrcThumbnails, DstArray);
}

void FVariantManager::RecordProperty(UPropertyValue* Prop) const
{
	if (!Prop)
	{
		return;
	}

	Prop->RecordDataFromResolvedObject();
}

void FVariantManager::ApplyProperty(UPropertyValue* Prop) const
{
	if (!Prop)
	{
		return;
	}

	Prop->ApplyDataToResolvedObject();
}

void FVariantManager::CallDirectorFunction(FName FunctionName, UVariantObjectBinding* TargetObject) const
{
	if (TargetObject && !FunctionName.IsNone())
	{
		TargetObject->ExecuteTargetFunction(FunctionName);
	}
}

void FVariantManager::CanAddActorsToVariant(const TArray<TWeakObjectPtr<AActor>>& InActors, const UVariant* InVariant, TArray<TWeakObjectPtr<AActor>>& OutActorsWeCanAdd)
{
	TSet<UObject*> BoundObjects;
	for (UVariantObjectBinding* Binding : InVariant->GetBindings())
	{
		BoundObjects.Add(Binding->GetObject());
	}

	OutActorsWeCanAdd.Empty();
	for (const TWeakObjectPtr<AActor>& Actor : InActors)
	{
		if (!BoundObjects.Contains(Actor.Get()))
		{
			OutActorsWeCanAdd.Add(Actor);
		}
	}
}

void FVariantManager::CanAddActorsToVariant(const TArray<UObject*>& InActors, const UVariant* InVariant, TArray<UObject*>& OutActorsWeCanAdd)
{
	TSet<UObject*> BoundObjects;
	for (UVariantObjectBinding* Binding : InVariant->GetBindings())
	{
		BoundObjects.Add(Binding->GetObject());
	}

	OutActorsWeCanAdd.Empty();
	for (UObject* Actor : InActors)
	{
		if (!BoundObjects.Contains(Actor))
		{
			OutActorsWeCanAdd.Add(Actor);
		}
	}
}

void FVariantManager::PostUndo(bool bSuccess)
{
	if (VariantManagerWidget.IsValid())
	{
		VariantManagerWidget->RefreshVariantTree();
		VariantManagerWidget->RefreshActorList();
		VariantManagerWidget->RefreshPropertyList();
	}
}

void FVariantManager::CaptureNewProperties(const TArray<UVariantObjectBinding*>& Bindings)
{
	// When adding actors to multiple variants at once the same actor will spawn a separate binding for each
	// variant. This prevents us from iterating the same actors properties more than once
	TSet<UObject*> BoundObjects;
	for (UVariantObjectBinding* Binding : Bindings)
	{
		UObject* Object = Binding->GetObject();
		if (Object)
		{
			BoundObjects.Add(Object);
		}
	}

	TSharedPtr<SCaptureDialog> CaptureDialog = SCaptureDialog::OpenCaptureDialogAsModalWindow(ECaptureDialogType::Property, BoundObjects.Array());
	TArray<TSharedPtr<FCapturableProperty>> CapturedProperties = CaptureDialog->GetCurrentCheckedProperties();
	if (CaptureDialog->GetUserAccepted())
	{
		CreatePropertyCaptures(CapturedProperties, Bindings);
	}
}

void FVariantManager::GetCapturableProperties(const TArray<AActor*>& Actors, TArray<TSharedPtr<FCapturableProperty>>& OutProperties, FString TargetPropertyPath, bool bCaptureAllArrayIndices)
{
	TSet<UObject*> BoundObjects;
	for (AActor* Actor : Actors)
	{
		BoundObjects.Add(Actor);
	}

	TArray<UObject*> BoundObjectsArr = BoundObjects.Array();

	FVariantManagerPropertyCapturer::CaptureProperties(BoundObjectsArr, OutProperties, TargetPropertyPath, bCaptureAllArrayIndices);
}

void FVariantManager::GetCapturableProperties(const TArray<UClass*>& Classes, TArray<TSharedPtr<FCapturableProperty>>& OutProperties, FString TargetPropertyPath, bool bCaptureAllArrayIndices)
{
	TSet<UObject*> BoundObjects;
	for (UClass* Class : Classes)
	{
		BoundObjects.Add(Class);
	}

	TArray<UObject*> BoundObjectsArr = BoundObjects.Array();

	FVariantManagerPropertyCapturer::CaptureProperties(BoundObjectsArr, OutProperties, TargetPropertyPath, bCaptureAllArrayIndices);
}

TSharedPtr<SVariantManager> FVariantManager::GetVariantManagerWidget()
{	
	if (!VariantManagerWidget.IsValid())
	{
		VariantManagerWidget = SNew(SVariantManager, SharedThis(this));
	}

	return VariantManagerWidget;
}

void FVariantManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CurrentLevelVariantSets);
}

#undef LOCTEXT_NAMESPACE
