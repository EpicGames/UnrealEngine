// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerBlueprintLibrary.h"

#include "CapturableProperty.h"
#include "FunctionCaller.h"
#include "K2Node_FunctionEntry.h"
#include "LevelVariantSets.h"
#include "LevelVariantSetsActor.h"
#include "Trace/Trace.inl"
#include "Variant.h"
#include "UObject/TextProperty.h"
#include "VariantManager.h"
#include "VariantManagerContentEditorModule.h"
#include "VariantManagerLog.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariantManagerBlueprintLibrary)


TUniquePtr<FVariantManager> UVariantManagerBlueprintLibrary::VariantManager;

UVariantManagerBlueprintLibrary::UVariantManagerBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetVariantManager();
}

FVariantManager& UVariantManagerBlueprintLibrary::GetVariantManager()
{
	if (!VariantManager.IsValid())
	{
		VariantManager = MakeUnique<FVariantManager>();
	}

	return *VariantManager.Get();
}

ULevelVariantSets* UVariantManagerBlueprintLibrary::CreateLevelVariantSetsAsset(const FString& AssetName, const FString& AssetPath)
{
	IVariantManagerContentEditorModule& ContentEditorModule = FModuleManager::LoadModuleChecked<IVariantManagerContentEditorModule>(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
	return Cast<ULevelVariantSets>(ContentEditorModule.CreateLevelVariantSetsAsset(AssetName, AssetPath));
}

ALevelVariantSetsActor* UVariantManagerBlueprintLibrary::CreateLevelVariantSetsActor(ULevelVariantSets* LevelVariantSetsAsset)
{
	if (LevelVariantSetsAsset == nullptr)
	{
		return nullptr;
	}

	IVariantManagerContentEditorModule& ContentEditorModule = FModuleManager::LoadModuleChecked<IVariantManagerContentEditorModule>(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
	return Cast<ALevelVariantSetsActor>(ContentEditorModule.GetOrCreateLevelVariantSetsActor(LevelVariantSetsAsset));
}




void UVariantManagerBlueprintLibrary::AddVariantSet(ULevelVariantSets* LevelVariantSets, UVariantSet* VariantSet)
{
	if (LevelVariantSets == nullptr || VariantSet == nullptr)
	{
		return;
	}

	GetVariantManager().AddVariantSets({VariantSet}, LevelVariantSets);
}

void UVariantManagerBlueprintLibrary::AddVariant(UVariantSet* VariantSet, UVariant* Variant)
{
	if (VariantSet == nullptr || Variant == nullptr)
	{
		return;
	}

	GetVariantManager().AddVariants({Variant}, VariantSet);
}

void UVariantManagerBlueprintLibrary::AddActorBinding(UVariant* Variant, AActor* Actor)
{
	if (Variant == nullptr || Actor == nullptr)
	{
		return;
	}

	GetVariantManager().CreateObjectBindings({Actor}, {Variant});
}




void UVariantManagerBlueprintLibrary::RemoveVariantSet(ULevelVariantSets* LevelVariantSets, UVariantSet* VariantSet)
{
	if (LevelVariantSets == nullptr || VariantSet == nullptr)
	{
		return;
	}

	GetVariantManager().RemoveVariantSetsFromParent({VariantSet});
}

void UVariantManagerBlueprintLibrary::RemoveVariant(UVariantSet* VariantSet, UVariant* Variant)
{
	if (VariantSet == nullptr || Variant == nullptr)
	{
		return;
	}

	GetVariantManager().RemoveVariantsFromParent({Variant});
}

void UVariantManagerBlueprintLibrary::RemoveActorBinding(UVariant* Variant, AActor* Actor)
{
	if (Variant == nullptr || Actor == nullptr)
	{
		return;
	}

	UVariantObjectBinding* TargetBinding = nullptr;

	const TArray<UVariantObjectBinding*>& Bindings = Variant->GetBindings();
	for (UVariantObjectBinding* Binding : Bindings)
	{
		if (Binding->GetObject() == Actor)
		{
			TargetBinding = Binding;
			break;
		}
	}

	if (TargetBinding)
	{
		GetVariantManager().RemoveObjectBindingsFromParent({TargetBinding});
	}
}




void UVariantManagerBlueprintLibrary::RemoveVariantSetByName(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName)
{
	if (LevelVariantSets == nullptr)
	{
		return;
	}

	if (UVariantSet* VarSet = LevelVariantSets->GetVariantSetByName(VariantSetName))
	{
		GetVariantManager().RemoveVariantSetsFromParent({VarSet});
	}
}

void UVariantManagerBlueprintLibrary::RemoveVariantByName(UVariantSet* VariantSet, const FString& VariantName)
{
	if (VariantSet == nullptr)
	{
		return;
	}

	if (UVariant* Var = VariantSet->GetVariantByName(VariantName))
	{
		GetVariantManager().RemoveVariantsFromParent({Var});
	}
}

void UVariantManagerBlueprintLibrary::RemoveActorBindingByName(UVariant* Variant, const FString& ActorName)
{
	if (Variant == nullptr)
	{
		return;
	}

	if (UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		GetVariantManager().RemoveObjectBindingsFromParent({Binding});
	}
}




void UVariantManagerBlueprintLibrary::Record(UPropertyValue* PropVal)
{
	if (PropVal == nullptr)
	{
		return;
	}

	PropVal->RecordDataFromResolvedObject();
}

void UVariantManagerBlueprintLibrary::Apply(UPropertyValue* PropVal)
{
	if (PropVal == nullptr)
	{
		return;
	}

	PropVal->ApplyDataToResolvedObject();
}

using MapEntry = TPairInitializer<const UClass*&, const FString&>;

FString UVariantManagerBlueprintLibrary::GetPropertyTypeString(UPropertyValue* PropVal)
{
	if (PropVal == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Input UPropertyValue was nullptr!");
		return FString();
	}

	FFieldClass* PropClass = PropVal->GetPropertyClass();
	if (PropClass->IsChildOf(FStructProperty::StaticClass()))
	{
		if (UStruct* Struct = PropVal->GetStructPropertyStruct())
		{
			if ( Struct->GetFName() == NAME_Rotator )
			{
				return TEXT("rotator");
			}
			else if ( Struct->GetFName() == NAME_Color )
			{
				return TEXT("color");
			}
			else if ( Struct->GetFName() == NAME_LinearColor )
			{
				return TEXT("linear_color");
			}
			else if ( Struct->GetFName() == NAME_Vector )
			{
				return TEXT("vector");
			}
			else if ( Struct->GetFName() == NAME_Quat )
			{
				return TEXT("quat");
			}
			else if ( Struct->GetFName() == NAME_Vector4 )
			{
				return TEXT("vector4");
			}
			else if ( Struct->GetFName() == NAME_Vector2D )
			{
				return TEXT("vector2d");
			}
			else if ( Struct->GetFName() == NAME_IntPoint )
			{
				return TEXT("int_point");
			}
		}
	}
	else if (PropClass->IsChildOf(FNumericProperty::StaticClass()))
	{
		if (PropVal->IsNumericPropertyFloatingPoint())
		{
			return TEXT("float");
		}
		else
		{
			return TEXT("int");
		}
	}
	else if (PropClass->IsChildOf(FBoolProperty::StaticClass()))
	{
		return TEXT("bool");
	}
	else if (PropClass->IsChildOf(FStrProperty::StaticClass()) ||
		     PropClass->IsChildOf(FTextProperty::StaticClass()) ||
		     PropClass->IsChildOf(FNameProperty::StaticClass()))
	{
		return TEXT("string");
	}
	else if (PropClass->IsChildOf(FObjectProperty::StaticClass()) || PropClass->IsChildOf(FInterfaceProperty::StaticClass()))
	{
		return TEXT("object");
	}

	UE_LOGF(LogVariantManager, Error, "Invalid property type for UPropertyValue '%ls'!", *PropVal->GetFullDisplayString());
	return FString();
}


TArray<FString> UVariantManagerBlueprintLibrary::GetCapturableProperties(UObject* ActorOrClass)
{
	if (ActorOrClass == nullptr)
	{
		return {};
	}

	TArray<FString> Result;
	TArray<TSharedPtr<FCapturableProperty>> OutProps;
	const bool bCaptureAllArrayIndices = false;
	FString TargetPropertyPath;

	if (AActor* Actor = Cast<AActor>(ActorOrClass))
	{
		GetVariantManager().GetCapturableProperties({Actor}, OutProps, TargetPropertyPath, bCaptureAllArrayIndices);
	}
	else if (UClass* Class = Cast<UClass>(ActorOrClass))
	{
		GetVariantManager().GetCapturableProperties({Class}, OutProps, TargetPropertyPath, bCaptureAllArrayIndices);
	}

	Result.Reserve(OutProps.Num());
	for (const TSharedPtr<FCapturableProperty>& PropPtr : OutProps)
	{
		Result.Add(PropPtr->DisplayName);
	}

	return Result;
}

UPropertyValue* UVariantManagerBlueprintLibrary::CaptureProperty(UVariant* Variant, AActor* Actor, FString PropertyPath)
{
	if (Variant == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Variant was null!");
		return nullptr;
	}

	if (Actor == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Actor was null!");
		return nullptr;
	}

	if (PropertyPath.IsEmpty())
	{
		UE_LOGF(LogVariantManager, Error, "PropertyPath was empty!");
		return nullptr;
	}

	UVariantObjectBinding* TargetBinding = nullptr;

	const TArray<UVariantObjectBinding*>& Bindings = Variant->GetBindings();
	for (UVariantObjectBinding* Binding : Bindings)
	{
		if (Binding->GetObject() == Actor)
		{
			TargetBinding = Binding;
			break;
		}
	}

	if (TargetBinding == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Variant '%ls' does not have a binding to actor '%ls'. Use 'variant.add_actor_binding(actor)' to create one", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
		return nullptr;
	}

	const bool bCaptureAllArrayIndices = false;
	TArray<TSharedPtr<FCapturableProperty>> OutProps;
	GetVariantManager().GetCapturableProperties({Actor}, OutProps, PropertyPath, bCaptureAllArrayIndices);

	if (OutProps.Num() < 1)
	{
		UE_LOGF(LogVariantManager, Error, "Actor '%ls' does not have a property with path '%ls'!", *Actor->GetActorLabel(), *PropertyPath);
		return nullptr;
	}

	TArray<UPropertyValue*> CreatedProps = GetVariantManager().CreatePropertyCaptures(OutProps, {TargetBinding});
	if (CreatedProps.Num() > 0)
	{
		return CreatedProps[0];
	}

	return nullptr;
}

int32 UVariantManagerBlueprintLibrary::AddDependency( UVariant* Variant, FVariantDependency& Dependency )
{
	if ( Variant )
	{
		return Variant->AddDependency(Dependency);
	}

	return INDEX_NONE;
}

void UVariantManagerBlueprintLibrary::SetDependency( UVariant* Variant, int32 Index, FVariantDependency& Dependency )
{
	if ( Variant )
	{
		Variant->SetDependency(Index, Dependency);
	}
}

void UVariantManagerBlueprintLibrary::DeleteDependency( UVariant* Variant, int32 Index )
{
	if ( Variant )
	{
		Variant->DeleteDependency(Index);
	}
}

TArray<FVariantDependency> UVariantManagerBlueprintLibrary::GetDependencies(UVariant* Variant)
{
	TArray<FVariantDependency> Dependencies;

	if (Variant)
	{
		for (int Index = 0; Index < Variant->GetNumDependencies(); ++Index)
		{
			Dependencies.Add(Variant->GetDependency(Index));
		}
	}

	return Dependencies;
}

TArray<UPropertyValue*> UVariantManagerBlueprintLibrary::GetCapturedProperties(UVariant* Variant, AActor* Actor)
{
	if (Variant == nullptr || Actor == nullptr)
	{
		return {};
	}

	TArray<UPropertyValue*> Result;
	for (UVariantObjectBinding* Binding : Variant->GetBindings())
	{
		if (AActor* ValidActor = Cast<AActor>(Binding->GetObject()))
		{
			if (ValidActor == Actor)
			{
				for (UPropertyValue* Prop : Binding->GetCapturedProperties())
				{
					Result.Add(Prop);
				}
				break;
			}
		}
	}

	return Result;
}

void UVariantManagerBlueprintLibrary::RemoveCapturedProperty(UVariant* Variant, AActor* Actor, UPropertyValue* Property)
{
	if (Variant == nullptr || Actor == nullptr || Property == nullptr)
	{
		return;
	}

	GetVariantManager().RemovePropertyCapturesFromParent({Property});
}

void UVariantManagerBlueprintLibrary::RemoveCapturedPropertyByName(UVariant* Variant, AActor* Actor, FString PropertyPath)
{
	if (Variant == nullptr || Actor == nullptr)
	{
		return;
	}

	UVariantObjectBinding* const* FoundBindingPtr = Variant->GetBindings().FindByPredicate([Actor](const UVariantObjectBinding* Binding)
	{
		AActor* ThisActor = Cast<AActor>(Binding->GetObject());
		return ThisActor && ThisActor == Actor;
	});

	if (FoundBindingPtr)
	{
		const TArray<UPropertyValue*>& Properties = (*FoundBindingPtr)->GetCapturedProperties();
		UPropertyValue* const* FoundPropPtr = Properties.FindByPredicate([&PropertyPath](const UPropertyValue* ThisProp)
		{
			return ThisProp && ThisProp->GetFullDisplayString() == PropertyPath;
		});

		if (FoundPropPtr)
		{
			GetVariantManager().RemovePropertyCapturesFromParent({*FoundPropPtr});
		}
	}
}

template<typename T>
void SetPropertyValueImpl(UPropertyValue* Property, T InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	Property->SetRecordedData((uint8*)&InValue, sizeof(T));
}

// Gets the value of Property as a T. This getter deep copies the bytes of the
// value object and returns it. It is necessary for value types and structs, as using
// the version for ref types below would (I think) try to return the object at the
// actual property recorded data bytes
template<typename T>
T GetPropertyValueImpl(UPropertyValue* Property, T InDefaultValue)
{
	if (Property == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from an invalid property!");
		return InDefaultValue;
	}

	if (!Property->HasRecordedData())
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from property '%ls', which does not have any!", *Property->GetFullDisplayString());
		return InDefaultValue;
	}

	if (Property->GetValueSizeInBytes() < sizeof(T))
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from property '%ls', but size of property data is less then needed!", *Property->GetFullDisplayString());
		return InDefaultValue;
	}

	T Result;
	FMemory::Memcpy(&Result, Property->GetRecordedData().GetData(), sizeof(T));
	return Result;
}

// Gets the value of Property as a T. This getter interprets the stored data as
// a T and returns a copy of it calling its copy constructor. It is necessary for
// the string types (FString, FText, FName), which are reference types. Just
// deep-copying the bytes as in the value type version above would obviously not
// trigger the copying of resources these reference types manage (char arrays, etc)
template<typename T>
T GetPropertyValueRefTypeImpl(UPropertyValue* Property, T InDefaultValue)
{
	if (Property == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from an invalid property!");
		return InDefaultValue;
	}

	if (!Property->HasRecordedData())
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from property '%ls', which does not have any!", *Property->GetFullDisplayString());
		return InDefaultValue;
	}

	if (Property->GetValueSizeInBytes() != sizeof(T))
	{
		UE_LOGF(LogVariantManager, Error, "Tried to access recorded data from property '%ls' as a type with size %llu, but the recorded data has size %d!", *Property->GetFullDisplayString(), sizeof(T), Property->GetValueSizeInBytes());
		return InDefaultValue;
	}

	return *((const T*)Property->GetRecordedData().GetData());
}

void UVariantManagerBlueprintLibrary::SetValueBool(UPropertyValue* Property, bool InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	int32 SizeInValue = sizeof(bool);
	int32 PropValSize = Property->GetValueSizeInBytes();

	if (SizeInValue == PropValSize && Property->GetPropertyClass()->IsChildOf(FBoolProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a bool as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

bool UVariantManagerBlueprintLibrary::GetValueBool(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, false);
}

void UVariantManagerBlueprintLibrary::SetValueInt(UPropertyValue* Property, int32 InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	FFieldClass* PropertyClass = Property->GetPropertyClass();

	// Technically blueprint-exposed properties can only be int32 or uint8, but this should handle all
	// cases. The two most common are first
	if (PropertyClass->IsChildOf(FIntProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else if (PropertyClass->IsChildOf(FByteProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<uint8>(InValue));
	}
	else if (PropertyClass->IsChildOf(FUInt64Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<uint64>(InValue));
	}
	else if (PropertyClass->IsChildOf(FUInt32Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<uint32>(InValue));
	}
	else if (PropertyClass->IsChildOf(FUInt16Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<uint16>(InValue));
	}

	else if (PropertyClass->IsChildOf(FInt64Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<int64>(InValue));
	}
	else if (PropertyClass->IsChildOf(FInt16Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<int16>(InValue));
	}
	else if (PropertyClass->IsChildOf(FInt8Property::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<int8>(InValue));
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set an integer as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *PropertyClass->GetName());
	}
}

int32 UVariantManagerBlueprintLibrary::GetValueInt(UPropertyValue* Property)
{
	if (Property == nullptr)
	{
		return 0;
	}

	FFieldClass* PropertyClass = Property->GetPropertyClass();

	// Technically blueprint-exposed properties can only be int32 or uint8, but this should handle all
	// cases. The two most common are first
	if (PropertyClass->IsChildOf(FIntProperty::StaticClass()))
	{
		return GetPropertyValueImpl<int32>(Property, 0);
	}
	else if (PropertyClass->IsChildOf(FByteProperty::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<uint8>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FUInt64Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<uint64>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FUInt32Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<uint32>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FUInt16Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<uint16>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FInt64Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<int64>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FInt16Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<int16>(Property, 0));
	}
	else if (PropertyClass->IsChildOf(FInt8Property::StaticClass()))
	{
		return static_cast<int32>(GetPropertyValueImpl<int8>(Property, 0));
	}

	return 0;
}

void UVariantManagerBlueprintLibrary::SetValueFloat(UPropertyValue* Property, float InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	FFieldClass* PropertyClass = Property->GetPropertyClass();

	if (PropertyClass->IsChildOf(FFloatProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else if (PropertyClass->IsChildOf(FDoubleProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, static_cast<double>(InValue));
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a float as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *PropertyClass->GetName());
	}
}

float UVariantManagerBlueprintLibrary::GetValueFloat(UPropertyValue* Property)
{
	if (Property == nullptr)
	{
		return 0.0f;
	}

	FFieldClass* PropertyClass = Property->GetPropertyClass();

	if (PropertyClass->IsChildOf(FFloatProperty::StaticClass()))
	{
		return GetPropertyValueImpl<float>(Property, 0.0f);
	}
	else if (PropertyClass->IsChildOf(FDoubleProperty::StaticClass()))
	{
		return static_cast<float>(GetPropertyValueImpl<double>(Property, 0.0));
	}

	return 0.0f;
}

void UVariantManagerBlueprintLibrary::SetValueObject(UPropertyValue* Property, UObject* InValue)
{
	if (Property == nullptr || InValue == nullptr)
	{
		return;
	}

	FFieldClass* PropClass = Property->GetPropertyClass();
	if (PropClass->IsChildOf(FObjectProperty::StaticClass()) || PropClass->IsChildOf(FInterfaceProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set an UObject as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

UObject* UVariantManagerBlueprintLibrary::GetValueObject(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, (UObject*)nullptr);
}

void UVariantManagerBlueprintLibrary::SetValueString(UPropertyValue* Property, const FString& InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	if (Property->GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else if (Property->GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, FText::FromString(InValue));
	}
	else if (Property->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
	{
		SetPropertyValueImpl(Property, FName(*InValue));
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a string as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FString UVariantManagerBlueprintLibrary::GetValueString(UPropertyValue* Property)
{
	if (Property == nullptr)
	{
		return FString();
	}

	if (Property->GetPropertyClass()->IsChildOf(FStrProperty::StaticClass()))
	{
		return GetPropertyValueRefTypeImpl(Property, FString());
	}
	else if (Property->GetPropertyClass()->IsChildOf(FTextProperty::StaticClass()))
	{
		return GetPropertyValueRefTypeImpl(Property, FText()).ToString();
	}
	else if (Property->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
	{
		return GetPropertyValueRefTypeImpl(Property, FName()).ToString();
	}

	return FString();
}

void UVariantManagerBlueprintLibrary::SetValueRotator(UPropertyValue* Property, FRotator InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Rotator)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Rotator as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FRotator UVariantManagerBlueprintLibrary::GetValueRotator(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FRotator());
}

void UVariantManagerBlueprintLibrary::SetValueColor(UPropertyValue* Property, FColor InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Color)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Color as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FColor UVariantManagerBlueprintLibrary::GetValueColor(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FColor());
}

void UVariantManagerBlueprintLibrary::SetValueLinearColor(UPropertyValue* Property, FLinearColor InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_LinearColor)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a LinearColor as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FLinearColor UVariantManagerBlueprintLibrary::GetValueLinearColor(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FLinearColor());
}

void UVariantManagerBlueprintLibrary::SetValueVector(UPropertyValue* Property, FVector InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Vector)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Vector as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FVector UVariantManagerBlueprintLibrary::GetValueVector(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FVector());
}

void UVariantManagerBlueprintLibrary::SetValueQuat(UPropertyValue* Property, FQuat InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Quat)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Quat as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FQuat UVariantManagerBlueprintLibrary::GetValueQuat(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FQuat());
}

void UVariantManagerBlueprintLibrary::SetValueVector4(UPropertyValue* Property, FVector4 InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Vector4)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Vector4 as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FVector4 UVariantManagerBlueprintLibrary::GetValueVector4(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FVector4());
}

void UVariantManagerBlueprintLibrary::SetValueVector2D(UPropertyValue* Property, FVector2D InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_Vector2D)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set a Vector2D as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FVector2D UVariantManagerBlueprintLibrary::GetValueVector2D(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FVector2D());
}

void UVariantManagerBlueprintLibrary::SetValueIntPoint(UPropertyValue* Property, FIntPoint InValue)
{
	if (Property == nullptr)
	{
		return;
	}

	UScriptStruct* Struct = Property->GetStructPropertyStruct();
	if (Struct && Struct->GetFName() == NAME_IntPoint)
	{
		SetPropertyValueImpl(Property, InValue);
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Cannot set an IntPoint as the value of '%ls', which is a %ls!", *Property->GetFullDisplayString(), *Property->GetPropertyClass()->GetName());
	}
}

FIntPoint UVariantManagerBlueprintLibrary::GetValueIntPoint(UPropertyValue* Property)
{
	return GetPropertyValueImpl(Property, FIntPoint());
}

UBlueprint* UVariantManagerBlueprintLibrary::GetOrCreateDirectorBlueprint(UVariant* Variant)
{
	if (Variant)
	{
		if (UVariantSet* VariantSet = Variant->GetParent())
		{
			if (ULevelVariantSets* LevelVariantSets = VariantSet->GetParent())
			{
				GetVariantManager().InitVariantManager(LevelVariantSets);

				return GetVariantManager().GetOrCreateDirectorBlueprint(LevelVariantSets);
			}

			UE_LOGF(LogVariantManager, Error, "VariantSet '%ls' does not have a parent LevelVariantSets!", *VariantSet->GetDisplayText().ToString());
		}
		else
		{
			UE_LOGF(LogVariantManager, Error, "Variant '%ls' does not have a parent VariantSet!", *Variant->GetDisplayText().ToString());
		}
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Variant was null!");
	}
	
	return nullptr;
}

TArray<FName> UVariantManagerBlueprintLibrary::GetFunctionCallerNames(UVariant* Variant, AActor* Actor)
{
	if (Variant == nullptr || Actor == nullptr)
	{
		UE_LOGF(LogVariantManager, Error, "Variant or Actor was null!");

		return {};
	}

	TArray<FName> FunctionCallerNames;

	const FString ActorName = Actor->GetName();
	if (UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		const TArray<FFunctionCaller>& FunctionCallers = Binding->GetFunctionCallers();

		Algo::Transform(FunctionCallers, FunctionCallerNames, [](const FFunctionCaller& Caller)
		{
			return Caller.FunctionName;
		});
	}
	else
	{
		UE_LOGF(LogVariantManager, Warning, "Variant '%ls' does not have a binding for actor '%ls'!", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}

	return FunctionCallerNames;
}

bool UVariantManagerBlueprintLibrary::GetFunctionCallerArguments(
	UVariant* Variant, 
	AActor* Actor, 
	FName FunctionName,
	TMap<FName, FString>& Arguments)
{
	Arguments.Empty();

	if (Variant == nullptr || Actor == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant, Actor, or FunctionName was not specified!");
		return false;
	}

	const FString ActorName = Actor->GetName();
	if (const UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		const TArray<FFunctionCaller>& FunctionCallers = Binding->GetFunctionCallers();
		if (const FFunctionCaller* FoundCaller = FunctionCallers.FindByPredicate([&FunctionName](const FFunctionCaller& Caller)
		{
			return Caller.FunctionName == FunctionName;
		}))
		{
			Arguments = FoundCaller->FunctionArguments;
			return true;
		}

		UE_LOGF(LogVariantManager, Warning, "Function caller '%ls' not found in variant '%ls' for actor '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}
	else
	{
		UE_LOGF(LogVariantManager, Warning, "Variant '%ls' does not have a binding for actor '%ls'!", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}

	return false;
}

bool UVariantManagerBlueprintLibrary::CreateFunctionCaller(
	UVariant* Variant, 
	AActor* Actor, 
	FName FunctionName,
	EVariantFunctionCallerSignature Signature)
{
	if (Variant == nullptr || Actor == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant, Actor, or FunctionName was not specified!");
		return false;
	}
	
	const FString ActorName = Actor->GetName();
	UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName);
	if (Binding)
	{
		if (CheckTheFunctionSignature(Variant, Actor, FunctionName, Signature))
		{
			UE_LOGF(LogVariantManager, Display, "Function caller '%ls' with matching signature already exists in variant '%ls' for actor '%ls', forwarding to AddFunctionCaller...", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());

			return AddFunctionCaller(Variant, Actor, FunctionName);
		}
	}
	else
	{
		TArray<UVariantObjectBinding*> Bindings = GetVariantManager().CreateObjectBindings({Actor}, {Variant});
		if (Bindings.Num() == 1)
		{
			Binding = Bindings[0];
		}
	}

	if (Binding)
	{
		if (UVariantSet* VariantSet = Variant->GetParent())
		{
			if (ULevelVariantSets* LevelVariantSets = VariantSet->GetParent())
			{
				GetVariantManager().InitVariantManager(LevelVariantSets);

				if (UBlueprint* Director = GetVariantManager().GetOrCreateDirectorBlueprint(LevelVariantSets))
				{
					Director->Modify();
					if (UEdGraph* Graph = FBlueprintEditorUtils::CreateNewGraph(Director, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass()))
					{
						constexpr bool bIsUserCreated = true;
						FBlueprintEditorUtils::AddFunctionGraph<UClass>(Director, Graph, bIsUserCreated, nullptr);

						TArray<UK2Node_FunctionEntry*> EntryNodes;
						Graph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
						
						if (ensure(EntryNodes.Num() == 1 && EntryNodes[0]))
						{
							UK2Node_FunctionEntry* EntryNode = EntryNodes[0];

							int32 ExtraFunctionFlags = ( FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public );
							EntryNode->AddExtraFlags(ExtraFunctionFlags);

							EntryNode->bIsEditable = true;
							EntryNode->MetaData.Category = NSLOCTEXT("VariantManagerLibrary", "DefaultCategory", "Director functions");
							EntryNode->MetaData.bCallInEditor = false;

							EntryNode->NodeComment = TEXT("Will be called by the Variant Manager when switching variants.\nThe target pin will receive a reference to the actor to which\nthis function caller is bound.\n\nYou can modify this target pin, either removing it\nor changing its type to another object reference type.\nIf you do, however, in order for a reference to the bound actor to be assigned\nto the pin, the actor must be of a type derived from the pin type.");
							EntryNode->bCommentBubblePinned = true;
							EntryNode->bCommentBubbleVisible = true;

							switch (Signature)
							{
							case EVariantFunctionCallerSignature::Static:
								{
									EntryNode->AddExtraFlags(FUNC_BlueprintPure);
									EntryNode->ReconstructNode();
								}
								break;
							case EVariantFunctionCallerSignature::NoParameters:
								{}
								break;
							case EVariantFunctionCallerSignature::OneParameter:
								{
									FEdGraphPinType PinType;
									PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
									PinType.PinSubCategoryObject = UObject::StaticClass();
									EntryNode->CreateUserDefinedPin(TARGET_PIN_NAME, PinType, EGPD_Output, true);

									EntryNode->ReconstructNode();
								}
								break;
							case EVariantFunctionCallerSignature::OneParameterWithArgs:
								{
									{
										FEdGraphPinType PinType;
										PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
										PinType.PinSubCategoryObject = UObject::StaticClass();
										EntryNode->CreateUserDefinedPin(TARGET_PIN_NAME, PinType, EGPD_Output, true);
									}
									{
										FEdGraphPinType PinType;
										PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
										PinType.ContainerType = EPinContainerType::Map;
										PinType.PinSubCategory = UEdGraphSchema_K2::PC_Name;
										FEdGraphPinType ValuePinType;
										ValuePinType.PinCategory = UEdGraphSchema_K2::PC_String;
										PinType.PinValueType = FEdGraphTerminalType::FromPinType(ValuePinType);
										EntryNode->CreateUserDefinedPin(ARGUMENTS_PIN_NAME, PinType, EGPD_Output, true);
									}
									EntryNode->ReconstructNode();
								}
								break;
							case EVariantFunctionCallerSignature::FourParameters:
								{
									FEdGraphPinType PinType;
									PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
									PinType.PinSubCategoryObject = UObject::StaticClass();
									EntryNode->CreateUserDefinedPin(TARGET_PIN_NAME, PinType, EGPD_Output, true);

									PinType.PinSubCategoryObject = ULevelVariantSets::StaticClass();
									EntryNode->CreateUserDefinedPin(LEVEL_VARIANT_SETS_PIN_NAME, PinType, EGPD_Output, true);

									PinType.PinSubCategoryObject = UVariantSet::StaticClass();
									EntryNode->CreateUserDefinedPin(VARIANT_SET_PIN_NAME, PinType, EGPD_Output, true);

									PinType.PinSubCategoryObject = UVariant::StaticClass();
									EntryNode->CreateUserDefinedPin(VARIANT_PIN_NAME, PinType, EGPD_Output, true);

									EntryNode->ReconstructNode();
								}
								break;
							case EVariantFunctionCallerSignature::FourParametersWithArgs:
								{
									{
										FEdGraphPinType PinType;
										PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
										PinType.PinSubCategoryObject = UObject::StaticClass();
										EntryNode->CreateUserDefinedPin(TARGET_PIN_NAME, PinType, EGPD_Output, true);

										PinType.PinSubCategoryObject = ULevelVariantSets::StaticClass();
										EntryNode->CreateUserDefinedPin(LEVEL_VARIANT_SETS_PIN_NAME, PinType, EGPD_Output, true);

										PinType.PinSubCategoryObject = UVariantSet::StaticClass();
										EntryNode->CreateUserDefinedPin(VARIANT_SET_PIN_NAME, PinType, EGPD_Output, true);

										PinType.PinSubCategoryObject = UVariant::StaticClass();
										EntryNode->CreateUserDefinedPin(VARIANT_PIN_NAME, PinType, EGPD_Output, true);
									}
									{
										FEdGraphPinType PinType;
										PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
										PinType.ContainerType = EPinContainerType::Map;
										PinType.PinSubCategory = UEdGraphSchema_K2::PC_Name;
										FEdGraphPinType ValuePinType;
										ValuePinType.PinCategory = UEdGraphSchema_K2::PC_String;
										PinType.PinValueType = FEdGraphTerminalType::FromPinType(ValuePinType);
										EntryNode->CreateUserDefinedPin(ARGUMENTS_PIN_NAME, PinType, EGPD_Output, true);
									}

									EntryNode->ReconstructNode();
								}
								break;
							}

					        // Compile the blueprint
							FKismetEditorUtilities::CompileBlueprint(Director, EBlueprintCompileOptions::None, nullptr);

							return AddFunctionCaller(Variant, Actor, FunctionName);
						}
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogVariantManager, Error, "Failed to create a binding for actor '%ls' in variant '%ls' when trying to create a function caller!", *Actor->GetActorLabel(), *Variant->GetDisplayText().ToString());
	}

	return false;
}

bool UVariantManagerBlueprintLibrary::AddFunctionCaller(
	UVariant* Variant, 
	AActor* Actor, 
	FName FunctionName)
{
	if (Variant == nullptr || Actor == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant, Actor, or FunctionName was not specified!");
		return false;
	}

	const FString ActorName = Actor->GetName();
	if (UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		TArray<FFunctionCaller>& FunctionCallers = Binding->GetFunctionCallers();
		if (FFunctionCaller* FoundCaller = FunctionCallers.FindByPredicate([&FunctionName](const FFunctionCaller& Caller)
		{
			return Caller.FunctionName == FunctionName;
		}))
		{
			UE_LOGF(LogVariantManager, Warning, "Function caller '%ls' already exists in variant '%ls' for actor '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());

			return false;
		}

		if (UVariantSet* VariantSet = Variant->GetParent())
		{
			if (ULevelVariantSets* LevelVariantSets = VariantSet->GetParent())
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(LevelVariantSets->GetDirectorGeneratedBlueprint()))
				{
					UK2Node_FunctionEntry* ThisFunc = nullptr;

					TArray<UK2Node_FunctionEntry*> EntryNodes;
					for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
					{
						if (FunctionGraph->GetName() == FunctionName)
						{
							EntryNodes.Reset();
							FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
							if (EntryNodes.Num() == 1)
							{
								if (UK2Node_FunctionEntry* EntryNode = EntryNodes[0])
								{
									if (FFunctionCaller::IsValidFunction(EntryNode))
									{
										ThisFunc = EntryNode;
										break;
									}
								}
							}
						}
					}
					
					if (ThisFunc)
					{
						FFunctionCaller NewCaller;
						NewCaller.FunctionName = FunctionName;
						NewCaller.SetFunctionEntry(ThisFunc);
						Binding->Modify();
						Binding->AddFunctionCallers({ NewCaller });

						return true;
					}

					UE_LOGF(LogVariantManager, Warning, "Could not find function entry node for function '%ls' in the director blueprint for variant '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString());
				}

				UE_LOGF(LogVariantManager, Warning, "Function '%ls' does not match any supported signatures for function callers in variant '%ls' for actor '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
			}
			else
			{
				UE_LOGF(LogVariantManager, Error, "VariantSet '%ls' does not have a parent LevelVariantSets!", *VariantSet->GetDisplayText().ToString());
			}
		}
		else
		{
			UE_LOGF(LogVariantManager, Error, "Variant '%ls' does not have a parent VariantSet!", *Variant->GetDisplayText().ToString());
		}
	}
	else
	{
		UE_LOGF(LogVariantManager, Warning, "Variant '%ls' does not have a binding for actor '%ls'!", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}

	return false;
}

bool UVariantManagerBlueprintLibrary::UpdateFunctionCallerArguments(
	UVariant* Variant, 
	AActor* Actor, 
	FName FunctionName,
	const TMap<FName, FString>& Arguments)
{
	if (Variant == nullptr || Actor == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant, Actor, or FunctionName was not specified!");
		return false;
	}

	const FString ActorName = Actor->GetName();
	if (UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		TArray<FFunctionCaller>& FunctionCallers = Binding->GetFunctionCallers();
		if (FFunctionCaller* FoundCaller = FunctionCallers.FindByPredicate([&FunctionName](const FFunctionCaller& Caller)
		{
			return Caller.FunctionName == FunctionName;
		}))
		{
			Binding->Modify();
			FoundCaller->FunctionArguments = Arguments;
			return true;
		}

		UE_LOGF(LogVariantManager, Warning, "Function caller '%ls' not found in variant '%ls' for actor '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}
	else
	{
		UE_LOGF(LogVariantManager, Warning, "Variant '%ls' does not have a binding for actor '%ls'!", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}

	return false;
}

bool UVariantManagerBlueprintLibrary::RemoveFunctionCaller(
	UVariant* Variant, 
	AActor* Actor, 
	FName FunctionName)
{
	if (Variant == nullptr || Actor == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant, Actor, or FunctionName was not specified!");
		return false;
	}

	const FString ActorName = Actor->GetName();
	if (UVariantObjectBinding* Binding = Variant->GetBindingByName(ActorName))
	{
		TArray<FFunctionCaller>& FunctionCallers = Binding->GetFunctionCallers();
		if (FFunctionCaller* FoundCaller = FunctionCallers.FindByPredicate([&FunctionName](const FFunctionCaller& Caller)
		{
			return Caller.FunctionName == FunctionName;
		}))
		{
			Binding->Modify();
			Binding->RemoveFunctionCallers({ FoundCaller });
			return true;
		}

		UE_LOGF(LogVariantManager, Warning, "Function caller '%ls' not found in variant '%ls' for actor '%ls'!", *FunctionName.ToString(), *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}
	else
	{
		UE_LOGF(LogVariantManager, Warning, "Variant '%ls' does not have a binding for actor '%ls'!", *Variant->GetDisplayText().ToString(), *Actor->GetActorLabel());
	}

	return false;
}

namespace
{
	bool IsObjectParam(TFieldIterator<FProperty>& It)
	{
		if (It)
		{
			FProperty* Property = *It;

			// Check property flags to determine how it is used
			if (Property->HasAnyPropertyFlags(CPF_Parm))
			{
				// Check for input/output/return value
				if (!Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					// This is an input or output parameter
					if (Property->HasAnyPropertyFlags(CPF_OutParm) && !Property->HasAnyPropertyFlags(CPF_ConstParm))
					{
						// Non-const reference: Output parameter (or Input/Output in BP)
						return false;
					}
							                
					// By value or const reference: Input parameter
					if (Property->IsA(FObjectProperty::StaticClass()))
					{
						if (FObjectProperty* ObjectProp = CastField<FObjectProperty>(Property))
						{
							return true;
						}
					}
				}
			}
		}

		return false;
	}

	bool IsArgsParam(TFieldIterator<FProperty>& It)
	{
		if (It)
		{
			FProperty* Property = *It;

			// Check property flags to determine how it is used
			if (Property->HasAnyPropertyFlags(CPF_Parm))
			{
				// Check for input/output/return value
				if (!Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					// This is an input or output parameter
					if (Property->HasAnyPropertyFlags(CPF_OutParm) && !Property->HasAnyPropertyFlags(CPF_ConstParm))
					{
						// Non-const reference: Output parameter (or Input/Output in BP)
						return false;
					}

					if (FMapProperty* MapProp = CastField<FMapProperty>(Property))
					{
						if (CastField<FNameProperty>(MapProp->KeyProp) && 
							CastField<FStrProperty>(MapProp->ValueProp))
						{
							return true;
						}
					}
				}
			}
		}

		return false;
	}

	bool IsObjectParamOfClass(TFieldIterator<FProperty>& It, const UClass* Class)
	{
		if (It && Class)
		{
			FProperty* Property = *It;

			// Check property flags to determine how it is used
			if (Property->HasAnyPropertyFlags(CPF_Parm))
			{
				// Check for input/output/return value
				if (!Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					// This is an input or output parameter
					if (Property->HasAnyPropertyFlags(CPF_OutParm) && !Property->HasAnyPropertyFlags(CPF_ConstParm))
					{
						// Non-const reference: Output parameter (or Input/Output in BP)
						return false;
					}
							                
					// By value or const reference: Input parameter
					if (Property->IsA(FObjectProperty::StaticClass()))
					{
						if (FObjectProperty* ObjectProp = CastField<FObjectProperty>(Property))
						{
							if (ObjectProp->PropertyClass == Class)
							{
								return true;
							}
						}
					}
				}
			}
		}

		return false;
	}
}

bool UVariantManagerBlueprintLibrary::CheckTheFunctionSignature(UVariant* Variant, AActor* BoundActor, FName FunctionName, EVariantFunctionCallerSignature Signature)
{
	if (Variant == nullptr || FunctionName.IsNone() || !FunctionName.IsValid())
	{
		UE_LOGF(LogVariantManager, Error, "Variant or FunctionName was not specified!");
		return false;
	}

	if (UVariantSet* VariantSet = Variant->GetParent())
	{
		if (ULevelVariantSets* LevelVariantSets = VariantSet->GetParent())
		{
			if (UObject* DirectorInstance = LevelVariantSets->GetDirectorGeneratedClass())
			{
				if (UFunction* Function = DirectorInstance->FindFunction(FunctionName))
				{
					switch (Signature)
					{
					case EVariantFunctionCallerSignature::Static:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 0)
						{
							return true;
						}
						break;
					case EVariantFunctionCallerSignature::NoParameters:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) &&
							!Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 0)
						{
							return true;
						}
						break;
					case EVariantFunctionCallerSignature::OneParameter:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) &&
							!Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 1)
						{
						    TFieldIterator<FProperty> It(Function);

						    if (IsObjectParam(It))
						    {
							    return true;
						    }

						    return false;
						}
						break;
					case EVariantFunctionCallerSignature::OneParameterWithArgs:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) &&
							!Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 2)
						{
						    TFieldIterator<FProperty> It(Function);
							
							if (IsObjectParam(It))
						    {
								++It;
								if (IsArgsParam(It))
								{
									return true;
								}
							}
						}
						break;
					case EVariantFunctionCallerSignature::FourParameters:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) &&
							!Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 4)
						{
						    TFieldIterator<FProperty> It(Function);
							
							if (IsObjectParam(It))
						    {
								++It;
								if (IsObjectParamOfClass(It, ULevelVariantSets::StaticClass()))
								{
									++It;
									if (IsObjectParamOfClass(It, UVariantSet::StaticClass()))
									{
										++It;
										if (IsObjectParamOfClass(It, UVariant::StaticClass()))
										{
											return true;
										}
									}
								}
							}
						}
						break;
					case EVariantFunctionCallerSignature::FourParametersWithArgs:
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) &&
							!Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
							Function->NumParms == 5)
						{
						    TFieldIterator<FProperty> It(Function);
							
							if (IsObjectParam(It))
						    {
								++It;
								if (IsObjectParamOfClass(It, ULevelVariantSets::StaticClass()))
								{
									++It;
									if (IsObjectParamOfClass(It, UVariantSet::StaticClass()))
									{
										++It;
										if (IsObjectParamOfClass(It, UVariant::StaticClass()))
										{
											++It;
											if (IsArgsParam(It))
											{
												return true;
											}
										}
									}
								}
							}
						}
						break;
					}
				}
			}
		}
	}

	return false;
}
