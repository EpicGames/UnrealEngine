// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerClipboard.h"



TArray<TStrongObjectPtr<UVariant>> FVariantManagerClipboard::StoredVariants = TArray<TStrongObjectPtr<UVariant>>();
TArray<TStrongObjectPtr<UVariantSet>> FVariantManagerClipboard::StoredVariantSets = TArray<TStrongObjectPtr<UVariantSet>>();
TArray<TStrongObjectPtr<UVariantObjectBinding>> FVariantManagerClipboard::StoredObjectBindings = TArray<TStrongObjectPtr<UVariantObjectBinding>>();
