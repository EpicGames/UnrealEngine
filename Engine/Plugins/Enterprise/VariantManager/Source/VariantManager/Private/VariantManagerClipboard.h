// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"

#include "Variant.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"

class FVariantManagerClipboard
{
public:

	static void Push(UVariant* Variant)
	{
		StoredVariants.Add(TStrongObjectPtr<UVariant>(Variant));
	}
	static void Push(UVariantSet* VariantSet)
	{
		StoredVariantSets.Add(TStrongObjectPtr<UVariantSet>(VariantSet));
	}
	static void Push(UVariantObjectBinding* ObjectBinding)
	{
		StoredObjectBindings.Add(TStrongObjectPtr<UVariantObjectBinding>(ObjectBinding));
	}
	static void Push(const TArray<UVariant*>& Variants)
	{
		StoredVariants.Reserve(Variants.Num());
		for (UVariant* Variant : Variants)
		{
			Push(Variant);
		}
	}
	static void Push(const TArray<UVariantSet*>& VariantSets)
	{
		StoredVariantSets.Reserve(VariantSets.Num());
		for (UVariantSet* VariantSet : VariantSets)
		{
			Push(VariantSet);
		}
	}
	static void Push(const TArray<UVariantObjectBinding*>& ObjectBindings)
	{
		StoredObjectBindings.Reserve(ObjectBindings.Num());
		for (UVariantObjectBinding* ObjectBinding : ObjectBindings)
		{
			Push(ObjectBinding);
		}
	}
	static const TArray<TStrongObjectPtr<UVariant>>& GetVariants()
	{
		return StoredVariants;
	}
	static const TArray<TStrongObjectPtr<UVariantSet>>& GetVariantSets()
	{
		return StoredVariantSets;
	}
	static const TArray<TStrongObjectPtr<UVariantObjectBinding>>& GetObjectBindings()
	{
		return StoredObjectBindings;
	}
	static void EmptyVariants()
	{
		StoredVariants.Empty();
	}
	static void EmptyVariantSets()
	{
		StoredVariantSets.Empty();
	}
	static void EmptyObjectBindings()
	{
		StoredObjectBindings.Empty();
	}
	static void Empty()
	{
		EmptyVariants();
		EmptyVariantSets();
		EmptyObjectBindings();
	}

private:

	// These should be TStrongObjectPtr because if we're cutting and pasting, we might temporarily be the
	// only objects pointing at these items
	static TArray<TStrongObjectPtr<UVariant>> StoredVariants;
	static TArray<TStrongObjectPtr<UVariantSet>> StoredVariantSets;
	static TArray<TStrongObjectPtr<UVariantObjectBinding>> StoredObjectBindings;
};
