// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerDragDropOp.h"


#define LOCTEXT_NAMESPACE "VariantManagerDragDropOp"


TSharedRef<FVariantManagerDragDropOp> FVariantManagerDragDropOp::New(TArray<TSharedRef<FVariantManagerDisplayNode>>& InDraggedNodes)
{
	TSharedRef<FVariantManagerDragDropOp> NewOp = MakeShareable(new FVariantManagerDragDropOp);

	NewOp->DraggedNodes = InDraggedNodes;
	NewOp->Construct();

	return NewOp;
}

TArray<TSharedRef<FVariantManagerDisplayNode>>& FVariantManagerDragDropOp::GetDraggedNodes()
{
	return DraggedNodes;
}

#undef LOCTEXT_NAMESPACE
