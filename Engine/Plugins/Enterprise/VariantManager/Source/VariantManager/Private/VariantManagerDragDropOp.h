// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class FVariantManagerDisplayNode;


class FVariantManagerDragDropOp : public FDecoratedDragDropOp
{
public:

	DRAG_DROP_OPERATOR_TYPE(FVariantManagerDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FVariantManagerDragDropOp> New(TArray<TSharedRef<FVariantManagerDisplayNode>>& InDraggedNodes);

	TArray<TSharedRef<FVariantManagerDisplayNode>>& GetDraggedNodes();

private:

	// Force usage of factory method
	FVariantManagerDragDropOp() {}

private:

	TArray<TSharedRef<FVariantManagerDisplayNode>> DraggedNodes;
};
