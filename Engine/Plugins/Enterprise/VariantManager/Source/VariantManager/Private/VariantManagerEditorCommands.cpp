// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerEditorCommands.h"

#define LOCTEXT_NAMESPACE "FVariantManagerEditorCommands"


FVariantManagerEditorCommands::FVariantManagerEditorCommands()
	: TCommands<FVariantManagerEditorCommands>("FVariantManagerEditorCommands", LOCTEXT("VariantManagerEditor", "Variant Manager Editor"), NAME_None, "VariantManagerEditorStyle")
{
}

void FVariantManagerEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateVariantManagerCommand, "Add Variant Manager", "Create a new variant manager asset, and place an instance of it in this level", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddVariantSetCommand, "Create new Variant Set", "Creates a new variant set at the bottom of the list", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddSelectedActorsCommand, "Add selected actors", "Creates a new variant set at the bottom of the list", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwitchOnSelectedVariantCommand, "Switch on", "Applies a single variant's values to all its captured properties", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateThumbnailCommand, "Set from viewport", "Sets the thumbnail to what is currently shown on the viewport", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectThumbnailAssetCommand, "Set from asset...", "Use an asset to serve as the thumbnail", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LoadThumbnailCommand, "Set from file", "Load an image to serve as the thumbnail", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearThumbnailCommand, "Clear", "Resets the thumbnail to default", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddPropertyCaptures, "Add properties", "Capture new properties from the selected actors to this variant", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddFunction, "Add function caller", "Add a function caller to execute blueprint code or trigger events when this variant is switched on", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RebindActorDisabled, "Rebind to other Actor", "Actors can only be rebound one at a time!", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RebindToSelected, "Bind selected", "Rebind to the first selected actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveActorBindings, "Remove", "Remove the actor binding from its variant", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyProperty, "Apply recorded value", "Applies the recorded value of this property to the bound actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RecordProperty, "Record current value", "Updates the recorded value with the current value of this property in the bound actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveCapture, "Remove capture", "Stop tracking this property", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CallFunction, "Call", "Call this director function", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveFunction, "Remove function caller", "Remove this function caller", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
