// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"


class FVariantManagerEditorCommands
	: public TCommands<FVariantManagerEditorCommands>
{
public:

	/** Default constructor. */
	FVariantManagerEditorCommands();

	/** Initialize commands */
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> CreateVariantManagerCommand;
	TSharedPtr<FUICommandInfo> AddVariantSetCommand;
	TSharedPtr<FUICommandInfo> AddSelectedActorsCommand;
	TSharedPtr<FUICommandInfo> SwitchOnSelectedVariantCommand;
	TSharedPtr<FUICommandInfo> CreateThumbnailCommand;
	TSharedPtr<FUICommandInfo> SelectThumbnailAssetCommand;
	TSharedPtr<FUICommandInfo> LoadThumbnailCommand;
	TSharedPtr<FUICommandInfo> ClearThumbnailCommand;
	TSharedPtr<FUICommandInfo> AddPropertyCaptures;
	TSharedPtr<FUICommandInfo> AddFunction;
	TSharedPtr<FUICommandInfo> RebindActorDisabled;
	TSharedPtr<FUICommandInfo> RebindToSelected;
	TSharedPtr<FUICommandInfo> RemoveActorBindings;
	TSharedPtr<FUICommandInfo> ApplyProperty;
	TSharedPtr<FUICommandInfo> RecordProperty;
	TSharedPtr<FUICommandInfo> RemoveCapture;
	TSharedPtr<FUICommandInfo> CallFunction;
	TSharedPtr<FUICommandInfo> RemoveFunction;
};
