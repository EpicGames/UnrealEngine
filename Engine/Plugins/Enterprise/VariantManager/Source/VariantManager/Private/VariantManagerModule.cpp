// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerModule.h"

#include "LevelVariantSets.h"
#include "LevelVariantSetsEditorToolkit.h"
#include "VariantManager.h"
#include "VariantManagerContentEditorModule.h"
#include "VariantManagerEditorCommands.h"
#include "VariantManagerStyle.h"

#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "VariantManagerUtils.h"

#define LOCTEXT_NAMESPACE "VariantManagerModule"

class FVariantManagerModule : public IVariantManagerModule
{
public:
	virtual void StartupModule() override
	{
		FVariantManagerEditorCommands::Register();

		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

		// Register a delegate to detect whenever we should open an editor for a LevelVariantSets asset, and relay the
		// data to LevelVariantSetsEditorToolkit, which will spawn a VariantManager
		IVariantManagerContentEditorModule& ContentEditorModule = FModuleManager::LoadModuleChecked<IVariantManagerContentEditorModule>(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
		FOnLevelVariantSetsEditor LevelVariantSetsEditorDelegate = FOnLevelVariantSetsEditor::CreateStatic( FVariantManagerModule::OnLevelVariantSetsEditor );
		ContentEditorModule.RegisterOnLevelVariantSetsDelegate( LevelVariantSetsEditorDelegate );

		// We need to register a tab spawner now so that old tabs that were open when we closed the editor can be reopened
		// correctly displaying the "Variant Manager" title
		// Sadly this code runs after the LevelEditorModule is loaded, but before it has created its TabManager. We need to
		// subscribe to this event then, so that as soon as its created we'll register this spawner
		OnTabManagerChangedSubscription = LevelEditorModule.OnTabManagerChanged().AddLambda([ &LevelEditorModule ]()
		{
			RegisterTabSpawner( LevelEditorModule.GetLevelEditorTabManager() );
		});

		// Make sure we update the cached FProperty pointers we use for exception properties whenever hot reload happens to a relevant class
		FVariantManagerUtils::RegisterForHotReload();
	}

	virtual void ShutdownModule() override
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditorModule.OnTabManagerChanged().Remove(OnTabManagerChangedSubscription);

		IVariantManagerContentEditorModule& ContentEditorModule = FModuleManager::LoadModuleChecked<IVariantManagerContentEditorModule>(VARIANTMANAGERCONTENTEDITORMODULE_MODULE_NAME);
		ContentEditorModule.UnregisterOnLevelVariantSetsDelegate();

		UnregisterTabSpawner( LevelEditorModule.GetLevelEditorTabManager() );

		FVariantManagerUtils::UnregisterForHotReload();

		FVariantManagerEditorCommands::Unregister();
	}

	virtual TSharedRef<FVariantManager> CreateVariantManager(ULevelVariantSets* InLevelVariantSets) override
	{
		TSharedRef<FVariantManager> VariantManager = MakeShareable(new FVariantManager);
		VariantManager->InitVariantManager(InLevelVariantSets);

		return VariantManager;
	}

	static void RegisterTabSpawner( const TSharedPtr< FTabManager >& TabManager )
	{
		if ( !TabManager )
		{
			return;
		}

		if ( TabManager->HasTabSpawner( FLevelVariantSetsEditorToolkit::GetVariantManagerTabID() ) )
		{
			UnregisterTabSpawner( TabManager );
		}

		const FName VariantManagerStyleSetName = FVariantManagerStyle::Get().GetStyleSetName();

		TabManager->RegisterTabSpawner( FLevelVariantSetsEditorToolkit::GetVariantManagerTabID(), FOnSpawnTab::CreateStatic( &FVariantManagerModule::HandleTabManagerSpawnTab ) )
			.SetDisplayName( LOCTEXT("VariantManagerMainTab", "Variant Manager") )
			.SetGroup( WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory() )
			.SetIcon( FSlateIcon(VariantManagerStyleSetName, "VariantManager.Icon.Small") );
	}

	static void UnregisterTabSpawner( const TSharedPtr< FTabManager >& TabManager )
	{
		if ( !TabManager )
		{
			return;
		}

		TabManager->UnregisterTabSpawner( FLevelVariantSetsEditorToolkit::GetVariantManagerTabID() );
	}

	static TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("VariantManagerMainTitle", "Variant Manager"))
			.TabColorScale( FLevelVariantSetsEditorToolkit::GetWorldCentricTabColorScaleStatic() )
			.ContentPadding(FMargin(0))
			.TabRole(ETabRole::PanelTab);
	}

	static void OnLevelVariantSetsEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& EditWithinLevelEditor, class ULevelVariantSets* LevelVariantSets)
	{
		TSharedRef<FLevelVariantSetsEditorToolkit> Toolkit = MakeShareable(new FLevelVariantSetsEditorToolkit());
		Toolkit->Initialize(Mode, EditWithinLevelEditor, LevelVariantSets);
	}

private:
	FDelegateHandle OnTabManagerChangedSubscription;
};

IMPLEMENT_MODULE(FVariantManagerModule, VariantManager);

#undef LOCTEXT_NAMESPACE
