// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerNodeTree.h"

#include "VariantObjectBinding.h"
#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DisplayNodes/VariantManagerVariantSetNode.h"
#include "LevelVariantSets.h"
#include "VariantManager.h"
#include "VariantSet.h"

void FVariantManagerNodeTree::Empty()
{
	RootNodes.Empty();
	FilteredNodes.Empty();
	HoveredNode = nullptr;
}

void FVariantManagerNodeTree::Update()
{
	 HoveredNode = nullptr;

	 Empty();

	 TArray<TSharedRef<FVariantManagerVariantSetNode>> VariantSetNodes;

	 // Go over variant sets
	 const TArray<UVariantSet*>& VariantSets = VariantManager.GetCurrentLevelVariantSets()->GetVariantSets();
	 for (UVariantSet* VariantSet : VariantSets)
	 {
		 TSharedRef<FVariantManagerVariantSetNode> NewVarSetNode = MakeShareable(new FVariantManagerVariantSetNode(*VariantSet, nullptr, SharedThis(this)));

		 for (UVariant* Variant : VariantSet->GetVariants())
		 {
			 TSharedRef<FVariantManagerVariantNode> NewVarNode = MakeShareable(new FVariantManagerVariantNode(*Variant, NewVarSetNode, SharedThis(this)));

			 NewVarSetNode->AddChild(NewVarNode);
			 NewVarNode->SetParent(NewVarSetNode);
		 }

		 VariantSetNodes.Add(NewVarSetNode);
	 }

	 RootNodes.Append(VariantSetNodes);

	 // Re-filter the tree after updating
	 FilterNodes( FilterString );
}

const TArray<TSharedRef<FVariantManagerDisplayNode>>& FVariantManagerNodeTree::GetRootNodes() const
{
	return RootNodes;
}

bool FVariantManagerNodeTree::IsNodeFiltered( const TSharedRef<const FVariantManagerDisplayNode> Node ) const
{
	return FilteredNodes.Contains( Node );
}

static void AddChildNodes(const TSharedRef<FVariantManagerDisplayNode>& StartNode, TSet<TSharedRef<const FVariantManagerDisplayNode>>& OutFilteredNodes)
{
	OutFilteredNodes.Add(StartNode);

	for (TSharedRef<FVariantManagerDisplayNode> ChildNode : StartNode->GetChildNodes())
	{
		AddChildNodes(ChildNode, OutFilteredNodes);
	}
}

static void AddFilteredNode(const TSharedRef<FVariantManagerDisplayNode>& StartNode, TSet<TSharedRef<const FVariantManagerDisplayNode>>& OutFilteredNodes)
{
	AddChildNodes(StartNode, OutFilteredNodes);
}

static void AddParentNodes(const TSharedRef<FVariantManagerDisplayNode>& StartNode, TSet<TSharedRef<const FVariantManagerDisplayNode>>& OutFilteredNodes)
{
	TSharedPtr<FVariantManagerDisplayNode> ParentNode = StartNode->GetParent();
	if (ParentNode.IsValid())
	{
		OutFilteredNodes.Add(ParentNode.ToSharedRef());
		AddParentNodes(ParentNode.ToSharedRef(), OutFilteredNodes);
	}
}

/**
 * Recursively filters nodes
 *
 * @param StartNode			The node to start from
 * @param FilterStrings		The filter strings which need to be matched
 * @param OutFilteredNodes	The list of all filtered nodes
 * @return Whether the text filter was passed
 */
static bool FilterNodesRecursive( FVariantManager& VariantManager, const TSharedRef<FVariantManagerDisplayNode>& StartNode, const TArray<FString>& FilterStrings, TSet<TSharedRef<const FVariantManagerDisplayNode>>& OutFilteredNodes )
{
	// assume the filter is acceptable
	bool bPassedTextFilter = true;

	// check each string in the filter strings list against
	for (const FString& String : FilterStrings)
	{
		if (!StartNode->GetDisplayName().ToString().Contains(String))
		{
			bPassedTextFilter = false;
			break;
		}
	}

	// whether the start node is in the filter
	bool bInFilter = false;

	if (bPassedTextFilter)
	{
		// This node is now filtered
		AddFilteredNode(StartNode, OutFilteredNodes);

		bInFilter = true;
	}

	// check each child node to determine if it is filtered
	const TArray<TSharedRef<FVariantManagerDisplayNode>>& ChildNodes = StartNode->GetChildNodes();

	for (const auto& Node : ChildNodes)
	{
		// Mark the parent as filtered if any child node was filtered
		bPassedTextFilter |= FilterNodesRecursive(VariantManager, Node, FilterStrings, OutFilteredNodes);

		if (bPassedTextFilter && !bInFilter)
		{
			AddParentNodes(Node, OutFilteredNodes);

			bInFilter = true;
		}
	}

	return bPassedTextFilter;
}

void FVariantManagerNodeTree::FilterNodes(const FString& InFilter)
{
	FilteredNodes.Empty();

	if (InFilter.IsEmpty())
	{
		// No filter
		FilterString.Empty();
	}
	else
	{
		// Build a list of strings that must be matched
		TArray<FString> FilterStrings;

		FilterString = InFilter;
		// Remove whitespace from the front and back of the string
		FilterString.TrimStartAndEndInline();
		FilterString.ParseIntoArray(FilterStrings, TEXT(" "), true /*bCullEmpty*/);

		for (auto It = RootNodes.CreateIterator(); It; ++It)
		{
			// Recursively filter all nodes, matching them against the list of filter strings.  All filter strings must be matched
			FilterNodesRecursive(VariantManager, *It, FilterStrings, FilteredNodes);
		}
	}
}

void FVariantManagerNodeTree::SetHoveredNode(const TSharedPtr<FVariantManagerDisplayNode>& InHoveredNode)
{
	if (InHoveredNode != HoveredNode)
	{
		HoveredNode = InHoveredNode;
	}
}

const TSharedPtr<FVariantManagerDisplayNode>& FVariantManagerNodeTree::GetHoveredNode() const
{
	return HoveredNode;
}
