// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FVariantManager;
class FVariantManagerDisplayNode;
class UMovieSceneTrack;
class UVariantObjectBinding;
class FVariantManagerActorNode;
class FVariantManagerPropertyNode;


class FVariantManagerNodeTree : public TSharedFromThis<FVariantManagerNodeTree>
{
public:

	DECLARE_MULTICAST_DELEGATE(FOnUpdated);

	FVariantManagerNodeTree( class FVariantManager& InVariantManager )
		: VariantManager( InVariantManager )
	{}

	void Empty();

	// Creates display nodes for all variant sets and variants that pass the filter
	// and adds them to RootNodes. Doesn't update anything else
	void Update();
	const TArray< TSharedRef<FVariantManagerDisplayNode> >& GetRootNodes() const;

	bool HasActiveFilter() const { return !FilterString.IsEmpty(); }
	bool IsNodeFiltered( const TSharedRef<const FVariantManagerDisplayNode> Node ) const;
	void FilterNodes( const FString& InFilter );

	FVariantManager& GetVariantManager() {return VariantManager;}

	void SetHoveredNode(const TSharedPtr<FVariantManagerDisplayNode>& InHoveredNode);
	const TSharedPtr<FVariantManagerDisplayNode>& GetHoveredNode() const;

private:

	TArray< TSharedRef<FVariantManagerDisplayNode> > RootNodes;
	TSet<TSharedRef<const FVariantManagerDisplayNode>> FilteredNodes;
	TSharedPtr<FVariantManagerDisplayNode> HoveredNode;

	FString FilterString;

	FVariantManager& VariantManager;
};
