// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerPropertyCapturer.h"

#include "Components/MeshComponent.h"
#include "Components/LightComponent.h"
#include "CapturableProperty.h"
#include "EdGraphSchema_K2.h"
#include "VariantManagerUtils.h"
#include "SwitchActor.h"

#define LOCTEXT_NAMESPACE "VariantManagerPropertyCapturer"

class FPropertyCaptureHelper
{
public:
	FPropertyCaptureHelper(const TArray<UObject*>& InObject, EPropertyValueCategory CategoriesToCapture, FString InTargetPropertyPath = FString(), bool bCaptureAllArrayIndices = false);

	// Captures properties that fit set categories from the given objects (both set in constructor)
	TArray<TSharedPtr<FCapturableProperty>>& CaptureProperties();

private:

	void CaptureActorExceptionProperties(const AActor* Actor, FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames);
	void CaptureComponentExceptionProperties(const UActorComponent* Component, FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames);

	bool CanCaptureProperty(const UStruct* ContainerClass, const FProperty* Property, FName& OutSetterFunctionName);
	void CaptureProp(FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames, FName PropertySetterName = FName(), EPropertyValueCategory InCaptureType = EPropertyValueCategory::Generic);

	void GetAllPropertyPathsRecursive(const void* ValuePtr, const UStruct* PropertySource, FPropertyPath PropertyPath, FString PrettyPathString, TArray<FString> ComponentNames);

private:

	// If this is not None, we will only capture the property corresponding to this path, if it can
	// be resolved. Use this to check if a random string is a valid property path for the objects
	FString TargetPropertyPath;
	TArray<FString> SegmentedTargetPropertyPath;

	// If true, we won't capture the property of or step into components that are not the root component
	bool bJustRootComponent;

	// If true and if we have a TargetPropertyPath, a path like 'Tags[2]' will lead to the capturing of all
	// array indices of 'Tags', so 'Tags[0]', 'Tags[1]', 'Tags[2]', etc. If false, only 'Tags[2]' will be captured.
	bool bCaptureAllArrayIndices;

	// Points to the parent instance object. We use this to determine if a component we stumble upon is
	// on our object's component hierarchy or on some other object's
	const UObject* CurrentOwnerObject;

	EPropertyValueCategory CategoriesToCapture;

	const TArray<UObject*>& TargetObjects;
	TArray<TSharedPtr<FCapturableProperty>> CapturedPropertyPaths;

	// Helper datastructures to prevent copying the same path twice. It is convenient to keep
	// CapturedPropertyPaths as an array of TSharedPtrs so that it can be used directly in widgets, so
	// we need these for quick comparisons. In addition, comparing via FString interacts well with our
	// exceptions for transforms, materials and visibility
	TSet<const UActorComponent*> CapturedComponents;
	TSet<FString> CapturedPaths;
};

FPropertyCaptureHelper::FPropertyCaptureHelper(const TArray<UObject*>& InObjects, EPropertyValueCategory InCategoriesToCapture, FString InTargetPropertyPath, bool bInCaptureAllArrayIndices)
	: TargetPropertyPath(InTargetPropertyPath)
	, bCaptureAllArrayIndices(bInCaptureAllArrayIndices)
	, CategoriesToCapture(InCategoriesToCapture)
	, TargetObjects(InObjects)
{
	// HACK: If we're aiming for something specific (e.g. visibility, transform, etc) we just care about the root component
	bJustRootComponent = !EnumHasAnyFlags(InCategoriesToCapture, EPropertyValueCategory::Generic);

	const TCHAR* Delimiter = PATH_DELIMITER;

	SegmentedTargetPropertyPath.Empty();
	TargetPropertyPath.ParseIntoArray(SegmentedTargetPropertyPath, &Delimiter, 1, true);
}

void FPropertyCaptureHelper::CaptureActorExceptionProperties(const AActor* Actor, FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames)
{
	if ( CapturedPaths.Contains( PrettyPathString ) )
	{
		return;
	}

	const ASwitchActor* SwitchActor = Cast<const ASwitchActor>( Actor );
	if ( SwitchActor &&
		(TargetPropertyPath.IsEmpty() || TargetPropertyPath == SWITCH_ACTOR_SELECTED_OPTION_NAME ) &&
		EnumHasAnyFlags( CategoriesToCapture, EPropertyValueCategory::Option ) )
	{
		FString DisplayString = PrettyPathString + SWITCH_ACTOR_SELECTED_OPTION_NAME;

		// Initialize the dialog row item to unchecked, like in CaptureProp
		const bool bChecked = false;

		// Capture directly here as we don't need to check the property path for this exception property (by calling CaptureProp), as we already know this exists for the switch actor
		TSharedPtr<FCapturableProperty> NewCapture = MakeShared<FCapturableProperty>( DisplayString, PropertyPath, ComponentNames, bChecked, FName(), EPropertyValueCategory::Option );
		CapturedPropertyPaths.Add( NewCapture );
		CapturedPaths.Add( PrettyPathString );
	}
}

void FPropertyCaptureHelper::CaptureComponentExceptionProperties(const UActorComponent* Component, FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames)
{
	if (const UMeshComponent* ComponentAsMeshComponent = Cast<const UMeshComponent>(Component))
	{
		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::Material))
		{
			int32 NumMats = ComponentAsMeshComponent->GetNumMaterials();
			const static FString MatString = FString(TEXT("Material["));

			FArrayProperty* OverrideMatsProp = FVariantManagerUtils::GetOverrideMaterialsProperty();
			PropertyPath.AddProperty(FPropertyInfo(OverrideMatsProp));

			FPropertyInfo LeafInfo(OverrideMatsProp->Inner);

			for (int32 Index = 0; Index < NumMats; Index++)
			{
				FString DisplayString = PrettyPathString + MatString + FString::FromInt(Index) + FString(TEXT("]"));

				LeafInfo.ArrayIndex = Index;
				PropertyPath.AddProperty(LeafInfo);
				ComponentNames.Add(FString()); // One for the outer, other for the inner
				ComponentNames.Add(FString());
				CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(), EPropertyValueCategory::Material);
				PropertyPath = *PropertyPath.TrimPath(1);
				ComponentNames.Pop();
				ComponentNames.Pop();
			}

			PropertyPath = *PropertyPath.TrimPath(1);
		}
	}

	if (const USceneComponent* ComponentAsSceneComponent = Cast<const USceneComponent>(Component))
	{
		FString DisplayString;

		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::RelativeLocation))
		{
			FStructProperty* RelativeLocationProp = FVariantManagerUtils::GetRelativeLocationProperty();

			PropertyPath.AddProperty(FPropertyInfo(RelativeLocationProp));
			ComponentNames.Add(FString());
			DisplayString = PrettyPathString + FString(TEXT("Relative Location"));
			CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(TEXT("SetRelativeLocation")), EPropertyValueCategory::RelativeLocation);
			PropertyPath = *PropertyPath.TrimPath(1);
			ComponentNames.Pop();
		}

		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::RelativeRotation))
		{
			FStructProperty* RelativeRotationProp = FVariantManagerUtils::GetRelativeRotationProperty();

			PropertyPath.AddProperty(FPropertyInfo(RelativeRotationProp));
			ComponentNames.Add(FString());
			DisplayString = PrettyPathString + FString(TEXT("Relative Rotation"));
			CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(TEXT("SetRelativeRotation")), EPropertyValueCategory::RelativeRotation);
			PropertyPath = *PropertyPath.TrimPath(1);
			ComponentNames.Pop();
		}

		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::RelativeScale3D))
		{
			FStructProperty* RelativeScale3DProp = FVariantManagerUtils::GetRelativeScale3DProperty();

			PropertyPath.AddProperty(FPropertyInfo(RelativeScale3DProp));
			ComponentNames.Add(FString());
			// Important that it has this display name, as this will prevent us from capturing it twice, as there is a capturable RelativeScale3D property
			DisplayString = PrettyPathString + FString(TEXT("Relative Scale 3D"));
			CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(TEXT("SetRelativeScale3D")), EPropertyValueCategory::RelativeScale3D);
			PropertyPath = *PropertyPath.TrimPath(1);
			ComponentNames.Pop();
		}

		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::Visibility))
		{
			FBoolProperty* VisibilityProp = FVariantManagerUtils::GetVisibilityProperty();

			PropertyPath.AddProperty(FPropertyInfo(VisibilityProp));
			ComponentNames.Add(FString());
			DisplayString = PrettyPathString + FString(TEXT("Visible"));
			CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(TEXT("SetVisibility")), EPropertyValueCategory::Visibility);
			PropertyPath = *PropertyPath.TrimPath(1);
			ComponentNames.Pop();
		}
	}

	if (const ULightComponent* ComponentAsLightComponent = Cast<const ULightComponent>(Component))
	{
		FString DisplayString;

		if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::Color))
		{
			FStructProperty* LightColorProp = FVariantManagerUtils::GetLightColorProperty();

			PropertyPath.AddProperty(FPropertyInfo(LightColorProp));
			ComponentNames.Add(FString());
			DisplayString = PrettyPathString + FString(TEXT("Light Color"));
			CaptureProp(PropertyPath, DisplayString, ComponentNames, FName(TEXT("SetLightColor")), EPropertyValueCategory::Color);
			PropertyPath = *PropertyPath.TrimPath(1);
			ComponentNames.Pop();
		}
	}
}

bool IsHiddenFunction(const UStruct* PropertyStructure, const FString& FunctionName)
{
	static const FName HideFunctionsName(TEXT("HideFunctions"));

	TArray<FString> HideFunctions;
	if (const UClass* Class = Cast<const UClass>(PropertyStructure))
	{
		Class->GetHideFunctions(HideFunctions);
	}

	return HideFunctions.Contains(FunctionName);
}

bool FPropertyCaptureHelper::CanCaptureProperty(const UStruct* ContainerClass, const FProperty* Property, FName& OutSetterFunctionName)
{
	if (EnumHasAnyFlags(CategoriesToCapture, EPropertyValueCategory::Generic))
	{
		// Writable in some way and not editor-only/deprecated
		if (Property->HasAllPropertyFlags(CPF_Edit | CPF_BlueprintVisible) && !Property->HasAnyPropertyFlags(CPF_EditorOnly | CPF_Deprecated | CPF_DisableEditOnInstance))
		{
			// EditAnywhere and BlueprintReadWrite
			if (!Property->HasAnyPropertyFlags(CPF_EditConst | CPF_BlueprintReadOnly))
			{
				OutSetterFunctionName = FName();
				return true;
			}
			// See if it has a function setter instead (e.g. Intensity property is BlueprintReadOnly, but the same component has a SetIntensity function that is BlueprintCallable)
			else if (Property->HasAllPropertyFlags(CPF_BlueprintReadOnly))
			{
				// Reference: SequencerObjectChangeListener.cpp
				FString PropertyVarName = Property->GetName();

				// This property fits in all criteria but can only be set during construction scripts, so it's not suitable
				if (PropertyVarName == TEXT("bAutoActivate"))
				{
					OutSetterFunctionName = FName();
					return false;
				}

				// If this is a bool property, strip off the 'b' so that the "Set" functions to be
				// found are, for example, "SetHidden" instead of "SetbHidden"
				if (Property->GetClass()->IsChildOf(FBoolProperty::StaticClass()))
				{
					PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
				}

				static const FString Set(TEXT("Set"));
				const FString FunctionString = Set + PropertyVarName;
				FName FunctionName = FName(*FunctionString);

				UFunction* Function = nullptr;
				if (const UClass* Class = Cast<const UClass>(ContainerClass))
				{
					Function = Class->FindFunctionByName(FunctionName);
				}

				static const FName DeprecatedFunctionName(TEXT("DeprecatedFunction"));
				bool bFoundValidFunction =
					Function &&
					!Function->HasMetaData(DeprecatedFunctionName) &&
					Function->HasAllFunctionFlags(FUNC_BlueprintCallable) &&
					!IsHiddenFunction(ContainerClass, FunctionString);

				// Check if the function has a parameter with the same type as the property
				// And all other parameters have default arguments
				if (bFoundValidFunction)
				{
					for( TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_OutParm|CPF_ReturnParm); ++It)
					{
						FProperty* PropertyParam = *It;
						checkSlow(PropertyParam); // Fix static analysis warning
						bool bParamCanBeHandled = false;

						if (PropertyParam->SameType(Property))
						{
							if (Property->GetClass()->IsChildOf(FStructProperty::StaticClass()))
							{
								if (CastField<FStructProperty>(Property)->Struct == CastField<FStructProperty>(PropertyParam)->Struct)
								{
									bParamCanBeHandled = true;
								}
							}
							else
							{
								bParamCanBeHandled = true;
							}
						}

						if(!bParamCanBeHandled)
						{
							FString Default;
							UEdGraphSchema_K2::FindFunctionParameterDefaultValue(Function, PropertyParam, Default);
							if (!Default.IsEmpty())
							{
								bParamCanBeHandled = true;
							}
						}

						if (!bParamCanBeHandled)
						{
							//UE_LOGF(LogVariantManager, Warning, "For property '%ls', can't handle function setter '%ls' param '%ls'", *Property->GetName(), *Function->GetName(), *PropertyParam->GetName());
							bFoundValidFunction = false;
							break;
						}
					}
				}

				if (bFoundValidFunction)
				{
					OutSetterFunctionName = FunctionName;
					return true;
				}
			}
		}
	}

	OutSetterFunctionName = FName();
	return false;
}

void FPropertyCaptureHelper::CaptureProp(FPropertyPath& PropertyPath, FString& PrettyPathString, TArray<FString>& ComponentNames, FName PropertySetterName, EPropertyValueCategory InCaptureType)
{
	if (CapturedPaths.Contains(PrettyPathString))
	{
		return;
	}

	// Slight hack here: We need the bCaptureAllArrayIndices trick because the events that we use to auto-expose
	// property captures don't give granularity as to which array index it was that was modified, meaning we must
	// capture them all. It is more annoying to determine the array information when constructing the path though, so we
	// just pass along the base path (e.g. 'Tags') and bCaptureAllArrayIndices=true, and FPropertyCaptureHelper is
	// in charge of capturing 'Tags[0]', 'Tags[1]', etc
	bool bFitsTarget = false;
	if (!TargetPropertyPath.IsEmpty() && PropertyPath.GetNumProperties() > 0)
	{
		FString StringToCompare;

		if (bCaptureAllArrayIndices && PropertyPath.GetLeafMostProperty().ArrayIndex != INDEX_NONE)
		{
			int32 OpenBracketIndex;
			PrettyPathString.FindLastChar(TEXT('['), OpenBracketIndex);
			if (OpenBracketIndex != INDEX_NONE)
			{
				StringToCompare = PrettyPathString.Left(OpenBracketIndex);
			}
		}
		else
		{
			StringToCompare = PrettyPathString;
		}

		bFitsTarget = (StringToCompare == TargetPropertyPath);
	}

	if (TargetPropertyPath.IsEmpty() || bFitsTarget)
	{
		TSharedPtr<FCapturableProperty> NewCapture = MakeShared<FCapturableProperty>(PrettyPathString, PropertyPath, ComponentNames, false, PropertySetterName, InCaptureType);
		CapturedPropertyPaths.Add(NewCapture);
		CapturedPaths.Add(PrettyPathString);
	}
}

void FPropertyCaptureHelper::GetAllPropertyPathsRecursive(const void* ValuePtr, const UStruct* PropertySource, FPropertyPath PropertyPath, FString PrettyPathString, TArray<FString> ComponentNames)
{
	FName PropertySetterName;

	//@todo variantmanager clean this up, tons of duplication
	for (TFieldIterator<FProperty> PropertyIterator(PropertySource); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (Property)
		{
			FString PrettyString = Property->GetDisplayNameText().ToString();

			// Add category if we're a inside a UStruct (completely arbitrary decision, but helps to handle
			// the properties inside the structs we do support: FPostProcessSettings and the CineCamera structs)
			if (const UScriptStruct* Struct = Cast<UScriptStruct>(PropertySource))
			{
					FString Category = Property->GetMetaData(TEXT("Category"));
					if (!Category.IsEmpty())
					{
						Category = Category.Replace(TEXT("|"), PATH_DELIMITER);

					int32 LastDelimiterIndex = Category.Find(PATH_DELIMITER, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
					FString LastCategorySegment = (LastDelimiterIndex == INDEX_NONE)? Category : Category.RightChop(LastDelimiterIndex+1);

					// Some categories have the exact same name as the struct or the property, which is not helpful
					// e.g. Component / Focus Settings / Focus Settings / Property
					// or Component / Focus Method / Focus Method
					if (!PrettyPathString.EndsWith(PATH_DELIMITER + Category + PATH_DELIMITER) &&
						LastCategorySegment != PrettyString)
					{
						PrettyString = Category + PATH_DELIMITER + PrettyString;
					}
				}
			}

			// Update current path position
			FPropertyInfo PropInfo = FPropertyInfo(Property);
			PropertyPath.AddProperty(PropInfo);

			PrettyPathString += PrettyString;
			ComponentNames.Add(FString());

			// Arrays of..
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ValuePtr));
				for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
				{
					// Update current path position
					PropertyPath.AddProperty(FPropertyInfo(ArrayProperty->Inner, Index));
					FString InnerPrettyString = FString::Printf(TEXT("[%d]"), Index);
					PrettyPathString += InnerPrettyString;
					ComponentNames.Add(FString());

					// ...structs
					if (FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
					{
						// Check the UArrayProperty's flags. The Inner's flags are unused for this
						if (CanCaptureProperty(PropertySource, ArrayProperty, PropertySetterName) &&
							FVariantManagerUtils::IsBuiltInStructProperty(ArrayProperty->Inner))
						{
							CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
						}
					}
					// ...objects
					else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
					{
						void* ObjectContainer = ArrayHelper.GetRawPtr(Index);
						UObject* Object = ObjectProperty->GetObjectPropertyValue(ObjectContainer);
						if (UActorComponent* ObjAsComponent = Cast<UActorComponent>(Object))
						{
							// This prevents us from jumping between different actors and capturing the same component more than once through
							// upward links (like AttachParent) or something else the user might have
							// Transient components are created during play mode (like billboards, etc)
							if (ObjAsComponent->GetOwner() == CurrentOwnerObject && !CapturedComponents.Contains(ObjAsComponent) && !ObjAsComponent->HasAnyFlags(RF_Transient))
							{
								FString NameString = ObjAsComponent->GetName();
								ComponentNames[ComponentNames.Num()-1] = NameString;

								// Replace inner array property name with component name
								FString PrettyPathWithComponentName = PrettyPathString;				// RootComponent/AttachChildren[0]
								PrettyPathWithComponentName.RemoveFromEnd(InnerPrettyString);		// RootComponent/AttachChildren
								PrettyPathWithComponentName.RemoveFromEnd(PrettyString);			// RootComponent/
								PrettyPathWithComponentName += NameString;							// RootComponent/Children[0] (Audio)
								PrettyPathWithComponentName += PATH_DELIMITER;						// RootComponent/Children[0] (Audio)/

								CapturedComponents.Add(ObjAsComponent);

								// Prevent us from capturing stuff from other components other than the root component if we're capturing
								// a specific property (e.g. we don't care about the visibility of some hierarchy children or other component
								// when capturing visibility specifically, we just care about the visibility of the root component)
								if (!bJustRootComponent)
								{
									CaptureComponentExceptionProperties(ObjAsComponent, PropertyPath, PrettyPathWithComponentName, ComponentNames);

									if (TargetPropertyPath.IsEmpty() || TargetPropertyPath.StartsWith(PrettyPathWithComponentName))
									{
										GetAllPropertyPathsRecursive(Object, Object->GetClass(), PropertyPath, PrettyPathWithComponentName, ComponentNames);
									}
								}
							}
						}
						// Check the UArrayProperty's flags. The Inner's flags are unused for this
						else if (CanCaptureProperty(PropertySource, ArrayProperty, PropertySetterName))
						{
							CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
						}
					}
					// ...simple properties
					// Check the UArrayProperty's flags. The Inner's flags are unused for this
					else if (CanCaptureProperty(PropertySource, ArrayProperty, PropertySetterName))
					{
						CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
					}

					PropertyPath = *PropertyPath.TrimPath(1);
					PrettyPathString.RemoveFromEnd(InnerPrettyString);
					ComponentNames.Pop();
				}
			}
			// Structs
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				bool bIsBuiltIn = FVariantManagerUtils::IsBuiltInStructProperty(Property);
				if (CanCaptureProperty(PropertySource, Property, PropertySetterName) && bIsBuiltIn)
				{
					CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
				}
				else if(!bIsBuiltIn)
				{
					UScriptStruct* Struct = StructProperty->Struct;
					const void* StructAddressInValuePtr = StructProperty->ContainerPtrToValuePtr<const void>(ValuePtr);

					if (FVariantManagerUtils::IsWalkableStructProperty(Property) &&
						!Property->HasAnyPropertyFlags(CPF_EditorOnly | CPF_Deprecated | CPF_DisableEditOnInstance))
					{
						PrettyPathString += PATH_DELIMITER;
						GetAllPropertyPathsRecursive(StructAddressInValuePtr, Struct, PropertyPath, PrettyPathString, ComponentNames);
						PrettyPathString.RemoveFromEnd(PATH_DELIMITER);
					}
				}
			}
			// Objects
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				const void* ObjectContainer = ObjectProperty->ContainerPtrToValuePtr<const void>(ValuePtr);
				const UObject* Object = ObjectProperty->GetObjectPropertyValue(ObjectContainer);
				if (const UActorComponent* ObjAsComponent = Cast<const UActorComponent>(Object))
				{
					// We just step into the root component property
					bool bIsRootComponentProperty = (ObjAsComponent->IsA(USceneComponent::StaticClass()) && Property->GetName() == TEXT("RootComponent"));

					// Prevent us from capturing stuff from other components other than the root component if we're capturing
					// a specific property (e.g. we don't care about the visibility of some hierarchy children or other component
					// when capturing visibility specifically, we just care about the visibility of the root component)
					bool bCanCaptureComponent = true;
					const AActor* OwnerActor = Cast<const AActor>(CurrentOwnerObject);
					if (OwnerActor->IsValidLowLevel() && bJustRootComponent && ObjAsComponent != OwnerActor->GetRootComponent())
					{
						bCanCaptureComponent = false;
					}

					// This prevents us from jumping between different actors and capturing the same component more than once through
					// upward links (like AttachParent) or something else the user might have in a custom component
					// Transient components are created during play mode (like billboards, etc)
					if (ObjAsComponent->GetOwner() == CurrentOwnerObject &&
						!CapturedComponents.Contains(ObjAsComponent) &&
						!ObjAsComponent->HasAnyFlags(RF_Transient) &&
						bIsRootComponentProperty &&
						bCanCaptureComponent)
					{
						FString NameString = ObjAsComponent->GetName();
						ComponentNames[ComponentNames.Num()-1] = NameString;

						FString ComponentPrettyPath;
						if (!ObjAsComponent->HasAnyFlags(RF_DefaultSubObject))
						{
							// If we don't have a DefaultSubObject tag, we're a generated component for a blueprint, which can be
							// renamed by the user. So let's use that name instead
							ComponentPrettyPath = NameString + PATH_DELIMITER;
						}
						else
						{
							// For DefaultSubObject components, the name can't be set by the user and is something like
							// StaticMeshComponent0, so we keep the property name, which should be "Static Mesh Component".
							ComponentPrettyPath = ObjAsComponent->GetClass()->GetDisplayNameText().ToString() + PATH_DELIMITER;
						}

						CapturedComponents.Add(ObjAsComponent);
						CaptureComponentExceptionProperties(ObjAsComponent, PropertyPath, ComponentPrettyPath, ComponentNames);

						if (TargetPropertyPath.IsEmpty() || TargetPropertyPath.StartsWith(ComponentPrettyPath))
						{
							GetAllPropertyPathsRecursive(Object, Object->GetClass(), PropertyPath, ComponentPrettyPath, ComponentNames);
						}
					}
				}
				else if (CanCaptureProperty(PropertySource, Property, PropertySetterName))
				{
					CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
				}
			}
			// Simple properties
			else if (CanCaptureProperty(PropertySource, Property, PropertySetterName))
			{
				CaptureProp(PropertyPath, PrettyPathString, ComponentNames, PropertySetterName);
			}

			PropertyPath = *PropertyPath.TrimPath(1);
			PrettyPathString.RemoveFromEnd(PrettyString);
			ComponentNames.Pop();
		}
	}
}

TArray<TSharedPtr<FCapturableProperty>>& FPropertyCaptureHelper::CaptureProperties()
{
	FPropertyPath InitialPropertyPath;
	FString InitialPrettyString;
	TArray<FString> InitialComponentNames;

	for (UObject* TargetObject : TargetObjects)
	{
		ensure(TargetObject->IsValidLowLevel());

		const void* TargetInstance = nullptr;
		const UStruct* TargetClass = nullptr;

		// If the passed in object is a class itself
		if (UClass* AsClass = Cast<UClass>(TargetObject))
		{
			const UObject* CDO = AsClass->GetDefaultObject();

			TargetClass = AsClass;
			TargetInstance = CDO;
			CurrentOwnerObject = CDO;
		}
		// If the passed in object is an instance of a class
		else
		{
			TargetClass = TargetObject->GetClass();
			TargetInstance = TargetObject;
			CurrentOwnerObject = TargetObject;
		}

		CaptureActorExceptionProperties((AActor*)TargetInstance, InitialPropertyPath, InitialPrettyString, InitialComponentNames);

		GetAllPropertyPathsRecursive(TargetInstance, TargetClass, InitialPropertyPath, InitialPrettyString, InitialComponentNames);
	}

	CapturedPropertyPaths.Sort([](const TSharedPtr<FCapturableProperty>& A, const TSharedPtr<FCapturableProperty>& B)
	{
		return A->DisplayName < B->DisplayName;
	});

	return CapturedPropertyPaths;
}

void FVariantManagerPropertyCapturer::CaptureProperties(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps, FString TargetPropertyPath, bool bCaptureAllArrayIndices)
{
	EPropertyValueCategory All = (EPropertyValueCategory)~0;

	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, All, TargetPropertyPath, bCaptureAllArrayIndices);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureVisibility(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, EPropertyValueCategory::Visibility);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureTransform(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture,
		EPropertyValueCategory::RelativeLocation | EPropertyValueCategory::RelativeRotation |  EPropertyValueCategory::RelativeScale3D);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureLocation(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, EPropertyValueCategory::RelativeLocation);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureRotation(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, EPropertyValueCategory::RelativeRotation);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureScale3D(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, EPropertyValueCategory::RelativeScale3D);
	OutCapturedProps = Helper.CaptureProperties();
}

void FVariantManagerPropertyCapturer::CaptureMaterial(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps)
{
	FPropertyCaptureHelper Helper = FPropertyCaptureHelper(InObjectsToCapture, EPropertyValueCategory::Material);
	OutCapturedProps = Helper.CaptureProperties();
}

#undef LOCTEXT_NAMESPACE
