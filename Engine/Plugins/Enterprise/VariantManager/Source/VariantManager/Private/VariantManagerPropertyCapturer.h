// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SCapturedPropertiesWidget;
class SCapturedActorsWidget;
struct FCapturableProperty;

/**
 * Utility for capturing useful properties of a given UObject, as well as
 * building a small UI listing them and allowing each to be picked
 */
class FVariantManagerPropertyCapturer
{
public:
	/**
	 * Returns in OutCaptureProps a FCapturableProperty struct for the union of each and any viable property we can find on all
	 * InObjectsToCapture.
	 * TargetPropertyPath: When this is given it will only capture properties that have this *exact* property path
	 * bCaptureAllArrayIndices: If true and if we have a TargetPropertyPath, a path like 'Tags[2]' will lead to the capturing of all
	 * 						    array indices of 'Tags', so 'Tags[0]', 'Tags[1]', 'Tags[2]', etc. If false, only 'Tags[2]' will be captured.
	 */
	static void CaptureProperties(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps, FString TargetPropertyPath = FString(), bool bCaptureAllArrayIndices = false);

	/**
	 * Returns in OutCaptureProps the FCapturableProperty corresponding to the Visibility property of all USceneComponents that
	 * we find in InObjectsToCapture
	 */
	static void CaptureVisibility(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);

	/**
	 * Returns in OutCaptureProps the FCapturablePropertys corresponding to the transform properties of all USceneComponents that
	 * we find in InObjectsToCapture
	 */
	static void CaptureTransform(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);
	static void CaptureLocation(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);
	static void CaptureRotation(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);
	static void CaptureScale3D(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);

	/**
	 * Returns in OutCaptureProps the FCapturablePropertys corresponding to the material properties of all UMeshComponents that
	 * we find in InObjectsToCapture
	 */
	static void CaptureMaterial(const TArray<UObject*>& InObjectsToCapture, TArray<TSharedPtr<FCapturableProperty>>& OutCapturedProps);
};
