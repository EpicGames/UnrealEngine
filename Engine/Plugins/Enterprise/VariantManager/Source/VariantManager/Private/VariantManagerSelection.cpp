// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerSelection.h"

#include "DisplayNodes/VariantManagerActorNode.h"
#include "DisplayNodes/VariantManagerVariantNode.h"
#include "DisplayNodes/VariantManagerVariantSetNode.h"

FVariantManagerSelection::FVariantManagerSelection()
	: SuspendBroadcastCount(0)
	, bOutlinerNodeSelectionChangedBroadcastPending(false)
{
}

TSet<TSharedRef<FVariantManagerDisplayNode>>& FVariantManagerSelection::GetSelectedOutlinerNodes()
{
	return SelectedOutlinerNodes;
}

TSet<TSharedRef<FVariantManagerActorNode>>& FVariantManagerSelection::GetSelectedActorNodes()
{
	return SelectedActorNodes;
}

void FVariantManagerSelection::GetSelectedVariantsAndVariantSets(TArray<UVariant*>& OutVariants, TArray<UVariantSet*>& OutVariantSets)
{
	for (const TSharedRef<FVariantManagerDisplayNode>& Node : SelectedOutlinerNodes)
	{
		if (Node->GetType() == EVariantManagerNodeType::Variant)
		{
			TSharedPtr<FVariantManagerVariantNode> NodeAsVar = StaticCastSharedRef<FVariantManagerVariantNode>(Node);
			if (NodeAsVar.IsValid())
			{
				OutVariants.Add(&NodeAsVar->GetVariant());
			}
		}
		else if (Node->GetType() == EVariantManagerNodeType::VariantSet)
		{
			TSharedPtr<FVariantManagerVariantSetNode> NodeAsVarSetNode = StaticCastSharedRef<FVariantManagerVariantSetNode>(Node);
			if (NodeAsVarSetNode.IsValid())
			{
				OutVariantSets.Add(&NodeAsVarSetNode->GetVariantSet());
			}
		}
	}
}

FVariantManagerSelection::FOnSelectionChanged& FVariantManagerSelection::GetOnOutlinerNodeSelectionChanged()
{
	return OnOutlinerNodeSelectionChanged;
}

FVariantManagerSelection::FOnSelectionChanged& FVariantManagerSelection::GetOnActorNodeSelectionChanged()
{
	return OnActorNodeSelectionChanged;
}

TArray<FGuid> FVariantManagerSelection::GetBoundObjectsGuids()
{
	TArray<FGuid> OutGuids;
	return OutGuids;
}

void FVariantManagerSelection::AddToSelection(TSharedRef<FVariantManagerDisplayNode> OutlinerNode)
{
	SelectedOutlinerNodes.Add(OutlinerNode);
	OutlinerNode->SetSelected(true);
	if ( IsBroadcasting() )
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::AddActorNodeToSelection(TSharedRef<FVariantManagerActorNode> ActorNode)
{
	SelectedActorNodes.Add(ActorNode);
	ActorNode->SetSelected(true);
	if (IsBroadcasting())
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::AddToSelection(const TArray<TSharedRef<FVariantManagerDisplayNode>>& OutlinerNodes)
{
	SelectedOutlinerNodes.Append(OutlinerNodes);
	for (const TSharedRef<FVariantManagerDisplayNode>& Node : OutlinerNodes)
	{
		Node->SetSelected(true);
	}

	if (IsBroadcasting())
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::AddActorNodeToSelection(const TArray<TSharedRef<FVariantManagerActorNode>>& ActorNodes)
{
	SelectedActorNodes.Append(ActorNodes);
	for (const TSharedRef<FVariantManagerActorNode>& Node : ActorNodes)
	{
		Node->SetSelected(true);
	}

	if (IsBroadcasting())
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::RemoveFromSelection(TSharedRef<FVariantManagerDisplayNode> OutlinerNode)
{
	SelectedOutlinerNodes.Remove(OutlinerNode);
	OutlinerNode->SetSelected(false);
	if ( IsBroadcasting() )
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::RemoveActorNodeFromSelection(TSharedRef<FVariantManagerActorNode> ActorNode)
{
	SelectedActorNodes.Remove(ActorNode);
	ActorNode->SetSelected(false);
	if (IsBroadcasting())
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::SetSelectionTo(const TArray<TSharedRef<FVariantManagerDisplayNode>>& OutlinerNodes, bool bFireBroadcast)
{
	TSet<TSharedRef<FVariantManagerDisplayNode>> InSet(OutlinerNodes);

	TSet<TSharedRef<FVariantManagerDisplayNode>> NodesToSelect = InSet.Difference(SelectedOutlinerNodes);
	for (TSharedRef<FVariantManagerDisplayNode>& NodeToSelect : NodesToSelect)
	{
		SelectedOutlinerNodes.Add(NodeToSelect);
		NodeToSelect->SetSelected(true);
	}

	TSet<TSharedRef<FVariantManagerDisplayNode>> NodesToDeSelect = SelectedOutlinerNodes.Difference(InSet);
	for (TSharedRef<FVariantManagerDisplayNode>& NodeToDeSelect : NodesToDeSelect)
	{
		SelectedOutlinerNodes.Remove(NodeToDeSelect);
		NodeToDeSelect->SetSelected(false);
	}

	if (bFireBroadcast && IsBroadcasting() && (NodesToSelect.Num() > 0 || NodesToDeSelect.Num() > 0))
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::SetActorNodeSelectionTo(const TArray<TSharedRef<FVariantManagerActorNode>>& ActorNodes, bool bFireBroadcast)
{
	TSet<TSharedRef<FVariantManagerActorNode>> InSet(ActorNodes);

	TSet<TSharedRef<FVariantManagerActorNode>> NodesToSelect = InSet.Difference(SelectedActorNodes);
	for (TSharedRef<FVariantManagerActorNode>& NodeToSelect : NodesToSelect)
	{
		SelectedActorNodes.Add(NodeToSelect);
		NodeToSelect->SetSelected(true);
	}

	TSet<TSharedRef<FVariantManagerActorNode>> NodesToDeselect = SelectedActorNodes.Difference(InSet);
	for (TSharedRef<FVariantManagerActorNode>& NodeToDeSelect : NodesToDeselect)
	{
		SelectedActorNodes.Remove(NodeToDeSelect);
		NodeToDeSelect->SetSelected(false);
	}

	if (bFireBroadcast && IsBroadcasting() && (NodesToSelect.Num() > 0 || NodesToDeselect.Num() > 0))
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}

TSet<FString>& FVariantManagerSelection::GetSelectedNodePaths()
{
	return SelectedNodePaths;
}

bool FVariantManagerSelection::IsSelected(const TSharedRef<FVariantManagerDisplayNode> OutlinerNode) const
{
	return SelectedOutlinerNodes.Contains(OutlinerNode);
}

void FVariantManagerSelection::Empty()
{
	EmptySelectedOutlinerNodes();
	EmptySelectedActorNodes();
}

void FVariantManagerSelection::EmptySelectedOutlinerNodes()
{
	if (!SelectedOutlinerNodes.Num())
	{
		return;
	}

	for (const TSharedRef<FVariantManagerDisplayNode>& Node : SelectedOutlinerNodes)
	{
		Node->SetSelected(false);
	}

	SelectedOutlinerNodes.Empty();
	if ( IsBroadcasting() )
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::EmptySelectedActorNodes()
{
	if (!SelectedActorNodes.Num())
	{
		return;
	}

	for (const TSharedRef<FVariantManagerActorNode>& Node : SelectedActorNodes)
	{
		Node->SetSelected(false);
	}

	SelectedActorNodes.Empty();
	if (IsBroadcasting())
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}

/** Suspend or resume broadcast of selection changing  */
void FVariantManagerSelection::SuspendBroadcast()
{
	SuspendBroadcastCount++;
}

void FVariantManagerSelection::ResumeBroadcast()
{
	SuspendBroadcastCount--;
	checkf(SuspendBroadcastCount >= 0, TEXT("Suspend/Resume broadcast mismatch!"));
}

bool FVariantManagerSelection::IsBroadcasting()
{
	return SuspendBroadcastCount == 0;
}

void FVariantManagerSelection::RequestOutlinerNodeSelectionChangedBroadcast()
{
	if (IsBroadcasting())
	{
		OnOutlinerNodeSelectionChanged.Broadcast();
	}
}

void FVariantManagerSelection::RequestActorNodeSelectionChangedBroadcast()
{
	if (IsBroadcasting())
	{
		OnActorNodeSelectionChanged.Broadcast();
	}
}


