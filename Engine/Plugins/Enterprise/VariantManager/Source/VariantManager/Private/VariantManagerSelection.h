// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FVariantManagerActorNode;
class FVariantManagerDisplayNode;

class UVariant;
class UVariantSet;


class FVariantManagerSelection
{
public:

	DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged)

public:

	FVariantManagerSelection();

	/** Gets a set of the selected outliner nodes. */
	TSet<TSharedRef<FVariantManagerDisplayNode>>& GetSelectedOutlinerNodes();
	TSet<TSharedRef<FVariantManagerActorNode>>& GetSelectedActorNodes();

	void GetSelectedVariantsAndVariantSets(TArray<UVariant*>& OutVariants, TArray<UVariantSet*>& OutVariantSets);

	void AddToSelection(TSharedRef<FVariantManagerDisplayNode> OutlinerNode);
	void AddToSelection(const TArray<TSharedRef<FVariantManagerDisplayNode>>& OutlinerNodes);

	void AddActorNodeToSelection(TSharedRef<FVariantManagerActorNode> ActorNode);
	void AddActorNodeToSelection(const TArray<TSharedRef<FVariantManagerActorNode>>& ActorNodes);

	void RemoveFromSelection(TSharedRef<FVariantManagerDisplayNode> OutlinerNode);
	void RemoveActorNodeFromSelection(TSharedRef<FVariantManagerActorNode> ActorNode);

	void SetSelectionTo(const TArray<TSharedRef<FVariantManagerDisplayNode>>& OutlinerNodes, bool bFireBroadcast = true);
	void SetActorNodeSelectionTo(const TArray<TSharedRef<FVariantManagerActorNode>>& ActorNodes, bool bFireBroadcast = true);

	// These manage paths to selected objects using ObjectNames
	// (e.g. ULevelVariantSets_2/UVariantSets_5/UVariant_1/UVariantObjectBinding_7)
	// Used to preserve selection during tree/list updates
	// (e.g. add a new object binding, add it to this, then refresh
	// tree to have it immediately selected)
	TSet<FString>& GetSelectedNodePaths();

	/** Returns whether or not the outliner node is selected. */
	bool IsSelected(const TSharedRef<FVariantManagerDisplayNode> OutlinerNode) const;

	/** Empties all selections. */
	void Empty();

	/** Empties the outliner node selection. */
	void EmptySelectedOutlinerNodes();
	void EmptySelectedActorNodes();

	/** Gets a multicast delegate which is called when the outliner node selection changes. */
	FOnSelectionChanged& GetOnOutlinerNodeSelectionChanged();
	FOnSelectionChanged& GetOnActorNodeSelectionChanged();

	/** Helper function to get an array of FGuid of bound objects on return */
	TArray<FGuid> GetBoundObjectsGuids();

	/** Suspend the broadcast of selection change notifications.  */
	void SuspendBroadcast();

	/** Resume the broadcast of selection change notifications.  */
	void ResumeBroadcast();

	void RequestOutlinerNodeSelectionChangedBroadcast();
	void RequestActorNodeSelectionChangedBroadcast();

private:
	/** When true, selection change notifications should be broadcasted. */
	bool IsBroadcasting();

private:

	// We load/store these to preserve selections when updating node trees/list views
	// objects can also manually set these so that the nodes built for these paths will be
	// selected when the tree is refreshed (e.g. add a new object binding, add it to this, then refresh
	// tree to have it immediately selected)
	TSet<FString> SelectedNodePaths;

	TSet<TSharedRef<FVariantManagerDisplayNode>> SelectedOutlinerNodes;
	TSet<TSharedRef<FVariantManagerActorNode>> SelectedActorNodes;

	FOnSelectionChanged OnOutlinerNodeSelectionChanged;
	FOnSelectionChanged OnActorNodeSelectionChanged;

	/** The number of times the broadcasting of selection change notifications has been suspended. */
	int32 SuspendBroadcastCount;

	/** When true there is a pending outliner node selection change which will be broadcast next tick. */
	bool bOutlinerNodeSelectionChangedBroadcastPending;
};
