// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"

FVariantManagerStyle::FVariantManagerStyle() : FSlateStyleSet("VariantManagerEditorStyle")
{
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("VariantManager")))
	{
		SetContentRoot(Plugin->GetContentDir());
	}

	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	/** Color used for the background of the entire variant manager as well as the spacer border */
	Set( "VariantManager.Panels.LightBackgroundColor", FLinearColor( FColor( 96, 96, 96, 255 ) ) );

	/** Color used as background for variant nodes, and background of properties and dependencies panels */
	Set( "VariantManager.Panels.ContentBackgroundColor", FLinearColor( FColor( 62, 62, 62, 255 ) ) );

	/** Color used for background of variant set nodes and panel headers, like Properties or Dependencies headers */
	Set( "VariantManager.Panels.HeaderBackgroundColor", FLinearColor( FColor( 48, 48, 48, 255 ) ) );

	/** Thickness of the light border around the entire variant manager tab and between items */
	Set( "VariantManager.Spacings.BorderThickness", 4.0f );

	/** The amount to indent child nodes of the layout tree */
	Set( "VariantManager.Spacings.IndentAmount", 10.0f );

	Set( "VariantManager.Icon.Small", new IMAGE_BRUSH_SVG("VariantManager", Icon16x16) );
	Set( "VariantManager.Icon", new IMAGE_BRUSH_SVG("VariantManager_64", Icon64x64) );
	Set( "VariantManager.VariantSet", new IMAGE_BRUSH_SVG("VariantSet_64", Icon64x64) );
	Set( "VariantManager.VariantSet.Small", new IMAGE_BRUSH_SVG("VariantSet_16", Icon16x16) );
	Set( "VariantManager.Variant", new IMAGE_BRUSH_SVG("Variant_64", Icon64x64) );
	Set( "VariantManager.Variant.Small", new IMAGE_BRUSH_SVG("Variant_16", Icon16x16) );

	Set( "VariantManager.AddFunction", new IMAGE_BRUSH_SVG("AddFunction_16", Icon16x16) );
	Set( "VariantManager.AddBinding", new IMAGE_BRUSH_SVG("AddBinding_16", Icon16x16) );

	Set("VariantManager.AutoCapture.Icon", new IMAGE_BRUSH_SVG("AutoCapture", Icon20x20));

	SetContentRoot(FPaths::EngineContentDir());
	
	Set("VariantManager.IconFavorite", new IMAGE_BRUSH_SVG("/Slate/Starship/Common/Favorite", Icon16x16));

	const FCheckBoxStyle RadioButtonStyle = FCheckBoxStyle()
		.SetUncheckedImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16))
		.SetUncheckedHoveredImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetUncheckedPressedImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetCheckedImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16))
		.SetCheckedHoveredImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
		.SetCheckedPressedImage(IMAGE_BRUSH_SVG("/Slate/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)));
	Set("VariantManager.VariantRadioButton", RadioButtonStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

const FName FVariantManagerStyle::GetAppStyleSetName()
{
	return FName("VariantManagerEditorStyle");
}
