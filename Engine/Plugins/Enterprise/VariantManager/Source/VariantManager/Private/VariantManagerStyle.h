// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

class FVariantManagerStyle final : public FSlateStyleSet
{
public:
	FVariantManagerStyle();

	~FVariantManagerStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static const FName GetAppStyleSetName();

	static FVariantManagerStyle& Get()
	{
		static FVariantManagerStyle Inst;
		return Inst;
	}
};
