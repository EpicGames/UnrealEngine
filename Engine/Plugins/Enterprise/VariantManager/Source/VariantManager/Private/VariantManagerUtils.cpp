// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantManagerUtils.h"

#include "CineCameraSettings.h"
#include "Components/MeshComponent.h"
#include "Components/LightComponent.h"
#include "CineCameraActor.h"	  // So we can check the CineCamera structs exist

#define GET_STRUCT_NAME_CHECKED(StructName) \
	((void)sizeof(F##StructName), TEXT(#StructName))

FArrayProperty* FVariantManagerUtils::OverrideMaterialsProperty = nullptr;
FStructProperty* FVariantManagerUtils::RelativeLocationProperty = nullptr;
FStructProperty* FVariantManagerUtils::RelativeRotationProperty = nullptr;
FStructProperty* FVariantManagerUtils::RelativeScale3DProperty = nullptr;
FBoolProperty* FVariantManagerUtils::VisiblityProperty = nullptr;
FStructProperty* FVariantManagerUtils::LightColorProperty = nullptr;
FStructProperty* FVariantManagerUtils::DefaultLightColorProperty = nullptr;
FDelegateHandle FVariantManagerUtils::OnHotReloadHandle;

void FVariantManagerUtils::RegisterForHotReload()
{
	OnHotReloadHandle = FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.AddStatic(&FVariantManagerUtils::InvalidateCache);
}

void FVariantManagerUtils::UnregisterForHotReload()
{
	FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.Remove(OnHotReloadHandle);
	OnHotReloadHandle.Reset();
}

bool FVariantManagerUtils::IsBuiltInStructProperty(const FProperty* Property)
{
	bool bIsBuiltIn = false;

		const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
	if (StructProp && StructProp->Struct)
	{
		FName StructName = StructProp->Struct->GetFName();

		bIsBuiltIn =
			StructName == NAME_Rotator ||
			StructName == NAME_Color ||
			StructName == NAME_LinearColor ||
			StructName == NAME_Vector ||
			StructName == NAME_Quat ||
			StructName == NAME_Vector4 ||
			StructName == NAME_Vector2D ||
			StructName == NAME_IntPoint;
	}

	return bIsBuiltIn;
}

bool FVariantManagerUtils::IsWalkableStructProperty(const FProperty* Property)
{
	const static TSet<FName> WalkableStructNames
	{
		GET_STRUCT_NAME_CHECKED(PostProcessSettings),
		GET_STRUCT_NAME_CHECKED(CameraLookatTrackingSettings),
		GET_STRUCT_NAME_CHECKED(CameraFilmbackSettings),
		GET_STRUCT_NAME_CHECKED(CameraLensSettings),
		GET_STRUCT_NAME_CHECKED(CameraFocusSettings),
		GET_STRUCT_NAME_CHECKED(CameraTrackingFocusSettings)
	};

	bool bIsWalkable = false;

	const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
	if (StructProp && StructProp->Struct)
	{
		FName StructName = StructProp->Struct->GetFName();

		bIsWalkable = WalkableStructNames.Contains(StructName);
	}

	return bIsWalkable;
}

FArrayProperty* FVariantManagerUtils::GetOverrideMaterialsProperty()
{
	if (!OverrideMaterialsProperty)
	{
		OverrideMaterialsProperty = FindFProperty<FArrayProperty>( UMeshComponent::StaticClass(), GET_MEMBER_NAME_CHECKED( UMeshComponent, OverrideMaterials ) );
	}

	return OverrideMaterialsProperty;
}

FStructProperty* FVariantManagerUtils::GetRelativeLocationProperty()
{
	if (!RelativeLocationProperty)
	{
		RelativeLocationProperty = FindFProperty<FStructProperty>( USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName() );
	}

	return RelativeLocationProperty;
}

FStructProperty* FVariantManagerUtils::GetRelativeRotationProperty()
{
	if (!RelativeRotationProperty)
	{
		RelativeRotationProperty = FindFProperty<FStructProperty>( USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName() );
	}

	return RelativeRotationProperty;
}

FStructProperty* FVariantManagerUtils::GetRelativeScale3DProperty()
{
	if (!RelativeScale3DProperty)
	{
		RelativeScale3DProperty = FindFProperty<FStructProperty>( USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName() );
	}

	return RelativeScale3DProperty;
}

FBoolProperty* FVariantManagerUtils::GetVisibilityProperty()
{
	if (!VisiblityProperty)
	{
		VisiblityProperty = FindFProperty<FBoolProperty>( USceneComponent::StaticClass(), USceneComponent::GetVisiblePropertyName() );
	}

	return VisiblityProperty;
}

FStructProperty* FVariantManagerUtils::GetLightColorProperty()
{
	if (!LightColorProperty)
	{
		LightColorProperty = FindFProperty<FStructProperty>( ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED( ULightComponent, LightColor ) );
	}

	return LightColorProperty;
}

void FVariantManagerUtils::InvalidateCache()
{
	OverrideMaterialsProperty = nullptr;
	RelativeLocationProperty = nullptr;
	RelativeRotationProperty = nullptr;
	RelativeScale3DProperty = nullptr;
	VisiblityProperty = nullptr;
	LightColorProperty = nullptr;
	DefaultLightColorProperty = nullptr;
}
#undef GET_MEMBER_NAME_CHECKED
