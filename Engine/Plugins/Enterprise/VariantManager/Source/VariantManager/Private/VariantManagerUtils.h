// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FArrayProperty;
class FBoolProperty;
class FDelegateHandle;
class FProperty;
class FStructProperty;

enum class EHotReloadedClassFlags;

class FVariantManagerUtils
{
public:
	// Invalidate our cached pointers whenever a hot reload happens, as the
	// classes that own those UPropertys might be reinstanced
	static void RegisterForHotReload();
	static void UnregisterForHotReload();

	// Returns true if Property is a FStructProperty with a Struct
	// of type FVector, FColor, FRotator, FQuat, etc
	static bool IsBuiltInStructProperty(const FProperty* Property);

	// Returns true if Property is a FStructProperty with a Struct
	// that the Variant Manager is allowed to capture specific properties from
	static bool IsWalkableStructProperty(const FProperty* Property);

	// Returns the OverrideMaterials property of the UMeshComponent class
	static FArrayProperty* GetOverrideMaterialsProperty();

	// Returns the RelativeLocation property of the USceneComponent class
	static FStructProperty* GetRelativeLocationProperty();

	// Returns the RelativeRotation property of the USceneComponent class
	static FStructProperty* GetRelativeRotationProperty();

	// Returns the RelativeScale3D property of the USceneComponent class
	static FStructProperty* GetRelativeScale3DProperty();

	// Returns the bVisible property of the USceneComponent class
	static FBoolProperty* GetVisibilityProperty();

	// Returns the LightColor property of the ULightComponent class
	static FStructProperty* GetLightColorProperty();

private:
	// Invalidates all of our cached FProperty pointers
	static void InvalidateCache();

	static FArrayProperty* OverrideMaterialsProperty;
	static FStructProperty* RelativeLocationProperty;
	static FStructProperty* RelativeRotationProperty;
	static FStructProperty* RelativeScale3DProperty;
	static FBoolProperty* VisiblityProperty;
	static FStructProperty* LightColorProperty;
	static FStructProperty* DefaultLightColorProperty;

	static FDelegateHandle OnHotReloadHandle;
};
