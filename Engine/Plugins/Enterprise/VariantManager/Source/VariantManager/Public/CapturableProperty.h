// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyPath.h"
#include "PropertyValue.h" // For ToCapturedPropSegmentArray

#include "CapturableProperty.generated.h"

enum class EPropertyValueCategory : uint8;

// Describes a property path that can be captured. It just exposes a display name but
// uses internal data in order to be able to capture exception properties, like materials
USTRUCT(BlueprintType)
struct FCapturableProperty
{
	GENERATED_USTRUCT_BODY()

	FCapturableProperty()
		: Checked(false)
		, Favorite(false)
		, CaptureType(EPropertyValueCategory::Undefined)
	{
	};
	FCapturableProperty(const FString& InDisplayName, const FPropertyPath& InPropPath, const TArray<FString>& InComponentNames, bool InChecked, FName InPropertySetterName, EPropertyValueCategory InCaptureType)
		: DisplayName(InDisplayName)
		, Prop(InPropPath)
		, Checked(InChecked)
		, Favorite(false)
		, CaptureType(InCaptureType)
		, ComponentNames(InComponentNames)
		, PropertySetterName(InPropertySetterName)
	{
		int32 Index = INDEX_NONE;
		PropertyName = InDisplayName;

		if (InDisplayName.FindLastChar(TEXT('/'), Index))
		{
			if (Index >= 1 && InDisplayName.Len() > Index)
			{
				SubPath = InDisplayName.Left(Index).TrimStartAndEnd();
				PropertyName = InDisplayName.Right(InDisplayName.Len() - Index - 1);
			}
		}

		if (SubPath.IsEmpty())
		{
			SubPath = TEXT("Actor");
			bIsActor = true;
		}
	}

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="VariantManager")
	FString DisplayName;

	FPropertyPath Prop;
	bool Checked;
	bool Favorite;
	bool bIsActor = false;
	EPropertyValueCategory CaptureType;
	TArray<FString> ComponentNames; // Same length as Prop, but will be empty for non-component ObjectProperties
	FName PropertySetterName;
	
	FString SubPath;		// Path without property name
	FString PropertyName;	// Just property name

	TArray<FCapturedPropSegment> ToCapturedPropSegmentArray()
	{
		int32 NumSegs = Prop.GetNumProperties();
		check(ComponentNames.Num() == NumSegs);

		TArray<FCapturedPropSegment> Result;
		Result.Reserve(NumSegs);

		for (int32 Index = 0; Index < NumSegs; Index++)
		{
			const FPropertyInfo& PropInfo = Prop.GetPropertyInfo(Index);

			FString& ComponentName = ComponentNames[Index];

			// If we have a component name, forget the index. This makes it easy to search by name later
			FCapturedPropSegment NewSegment;
			NewSegment.PropertyName = PropInfo.Property->GetName();
			NewSegment.PropertyIndex = ComponentName.IsEmpty() ? PropInfo.ArrayIndex : 0;
			NewSegment.ComponentName = ComponentName;

			Result.Add(NewSegment);
		}

		return Result;
	}
};
