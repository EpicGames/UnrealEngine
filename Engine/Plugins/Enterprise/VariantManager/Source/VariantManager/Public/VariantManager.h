// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class UVariant;
class UVariantSet;
class ULevelVariantSets;
class UPropertyValue;
class UVariantObjectBinding;
class SVariantManager;
class FVariantManagerSelection;
class FVariantManagerNodeTree;
struct FCapturableProperty;
class UK2Node_FunctionEntry;
struct FFunctionCaller;
class UBlueprint;
class FObjectThumbnail;

class FVariantManager final
	: public FGCObject
	, public FEditorUndoClient
	, public TSharedFromThis<FVariantManager>
{
public:

	FVariantManager();

	virtual ~FVariantManager();

	void Close();

	void InitVariantManager(ULevelVariantSets* InLevelVariantSets);

	void CaptureNewProperties(const TArray<UVariantObjectBinding*>& Bindings);
	void GetCapturableProperties(const TArray<AActor*>& Actors, TArray<TSharedPtr<FCapturableProperty>>& OutProperties, FString TargetPropertyPath = FString(), bool bCaptureAllArrayIndices = true);
	void GetCapturableProperties(const TArray<UClass*>& Classes, TArray<TSharedPtr<FCapturableProperty>>& OutProperties, FString TargetPropertyPath = FString(), bool bCaptureAllArrayIndices = true);

	// Sets up the blueprint class deriving from the function director that we'll use.
	// Do this here as it uses editor-only code
	UBlueprint* GetOrCreateDirectorBlueprint(ULevelVariantSets* InLevelVariantSets);
	UK2Node_FunctionEntry* CreateDirectorFunction(ULevelVariantSets* InLevelVariantSets, FName InFunctionName, UClass* PinClassType, bool bCreateWithArguments);
	UK2Node_FunctionEntry* CreateDirectorFunctionFromFunction(UFunction* QuickBindFunction, UClass* PinClassType, bool bCreateWithArguments);
	void CreateFunctionCaller(const TArray<UVariantObjectBinding*>& Bindings);

	// Adds existing items to existing containers
	void AddPropertyCaptures(const TArray<UPropertyValue*>& OfThisProperty, UVariantObjectBinding* ToThisBinding);
	void AddObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant, int32 InsertionIndex = INDEX_NONE, bool bReplaceOld = false);
	void AddVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex = INDEX_NONE);
	void AddVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVarSets, int32 InsertionIndex = INDEX_NONE);
	void AddFunctionCallers(const TArray<FFunctionCaller>& Functions, UVariantObjectBinding* Binding);

	// Removes existing items from existing containers
	void RemovePropertyCapturesFromParent(const TArray<UPropertyValue*>& ThisProp);
	void RemoveObjectBindingsFromParent(const TArray<UVariantObjectBinding*>& ThisBinding);
	void RemoveVariantsFromParent(const TArray<UVariant*>& ThisVariant);
	void RemoveVariantSetsFromParent(const TArray<UVariantSet*>& ThisVariantSet);
	void RemoveFunctionCallers(const TArray<FFunctionCaller*>& TheseCallers, UVariantObjectBinding* FromThisBinding);

	// Remove existing items from their parents and add them to existing containers
	void MoveObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant);
	void MoveVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex = INDEX_NONE);
	void MoveVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVariantSets, int32 InsertionIndex = INDEX_NONE);

	// Duplicate existing items to existing containers
	void DuplicateObjectBindings(const TArray<UVariantObjectBinding*>& TheseBindings, UVariant* ToThisVariant, int32 InsertionIndex = INDEX_NONE);
	void DuplicateVariants(const TArray<UVariant*>& TheseVariants, UVariantSet* ToThisVariantSet, int32 InsertionIndex = INDEX_NONE);
	void DuplicateVariantSets(const TArray<UVariantSet*>& TheseVariantSets, ULevelVariantSets* ToThisLevelVarSets, int32 InsertionIndex = INDEX_NONE);

	// Adds all children from one container to the other container, if it doesn't already have them
	VARIANTMANAGER_API void MergeObjectBindings(const UVariantObjectBinding* ThisBinding, UVariantObjectBinding* IntoThisBinding);
	VARIANTMANAGER_API void MergeVariants(const UVariant* ThisVar, UVariant* IntoThisVar);

	// Creates new items and add them to existing containers
	VARIANTMANAGER_API TArray<UPropertyValue*> CreatePropertyCaptures(const TArray<TSharedPtr<FCapturableProperty>>& OfTheseProperties, const TArray<UVariantObjectBinding*>& InTheseBindings, bool bSilenceWarnings = false);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateTransformPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateLocationPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateRotationPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateScale3DPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateVisibilityPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UPropertyValue*> CreateMaterialPropertyCaptures(const TArray<UVariantObjectBinding*>& InTheseBindings);
	VARIANTMANAGER_API TArray<UVariantObjectBinding*> CreateObjectBindings(const TArray<AActor*>& OfTheseActors, const TArray<UVariant*>& InTheseVariants, int32 InsertionIndex = INDEX_NONE);
	VARIANTMANAGER_API TArray<UVariantObjectBinding*> CreateObjectBindingsAndCaptures(const TArray<AActor*>& OfTheseActors, const TArray<UVariant*>& InTheseVariants, int32 InsertionIndex = INDEX_NONE);
	VARIANTMANAGER_API UVariant* CreateVariant(UVariantSet* InThisVariantSet);
	VARIANTMANAGER_API UVariantSet* CreateVariantSet(ULevelVariantSets* InThisLevelVariantSets);

	// Thumbnail operations
	// Returns one for each variant. These can be nullptr
	TArray<FObjectThumbnail*> GetVariantThumbnails(const TArray<UVariant*> SrcArray);
	// Expects one for each variant. Accepts nullptr (will clear any previous thumbnail when setting)
	void SetVariantThumbnails(const TArray<FObjectThumbnail*> Thumbnails, const TArray<UVariant*> Variants);
	void CopyVariantThumbnails(TArray<UVariant*> DstArray, const TArray<UVariant*> SrcArray);

	// Captured property handling
	void RecordProperty(UPropertyValue* Prop) const;
	void ApplyProperty(UPropertyValue* Prop) const;

	// Function handling
	void CallDirectorFunction(FName FunctionName, UVariantObjectBinding* TargetObject) const;

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	//~ End FEditorUndoClient Interface

	// Returns which actors we can add to an existing UVariant, since we only allow one per actor per variant
	// We can force add/replace with AddObjectBindings, but this allows us to check how many will be added, which
	// is used for tooltips and feedback
	void CanAddActorsToVariant(const TArray<TWeakObjectPtr<AActor>>& InActors, const UVariant* InVariant, TArray<TWeakObjectPtr<AActor>>& OutActorsWeCanAdd);
	void CanAddActorsToVariant(const TArray<UObject*>& InActors, const UVariant* InVariant, TArray<UObject*>& OutActorsWeCanAdd);

	TSharedPtr<SVariantManager> GetVariantManagerWidget();

	ULevelVariantSets* GetCurrentLevelVariantSets()
	{
		return CurrentLevelVariantSets.Get();
	}

	/** Gets the tree of nodes which is used to populate the animation outliner. */
	TSharedRef<FVariantManagerNodeTree> GetNodeTree()
	{
		return NodeTree;
	}

	FVariantManagerSelection& GetSelection()
	{
		return Selection.ToSharedRef().Get();
	}

	//~ FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FVariantManager");
	}

private:
	TSharedPtr<FVariantManagerSelection> Selection;

	TWeakObjectPtr<ULevelVariantSets> CurrentLevelVariantSets;

	TSharedPtr<SVariantManager> VariantManagerWidget;

	TSharedRef<FVariantManagerNodeTree> NodeTree;
};
