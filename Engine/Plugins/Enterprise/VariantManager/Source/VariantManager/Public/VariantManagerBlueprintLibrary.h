// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"


#include "PropertyValue.h"
#include "VariantManagerBlueprintLibrary.generated.h"

class FVariantManager;
struct FVariantDependency;

class ULevelVariantSets;
class ALevelVariantSetsActor;
class UVariantSet;
class UVariant;
class UPropertyValue;

UENUM(BlueprintType)
enum class EVariantFunctionCallerSignature : uint8
{
	Static,
	NoParameters,
	OneParameter,
	OneParameterWithArgs,
	FourParameters,
	FourParametersWithArgs
};

/**
* Library of functions that can be used by the Python API to trigger VariantManager operations
*/
UCLASS(meta=(ScriptName="VariantManagerLibrary"))
class UVariantManagerBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	// Gets or initializes and returns Variant Manager
	static FVariantManager& GetVariantManager();

public:

	// Creates a new LevelVariantSetsAsset named AssetName (e.g. 'MyLevelVariantSets') in the content path AssetPath (e.g. '/Game')
	UFUNCTION(BlueprintCallable, Category = "VariantManager")
	static ULevelVariantSets* CreateLevelVariantSetsAsset(const FString& AssetName, const FString& AssetPath);

	// Creates a new ALevelVariantSetsActor in the current scene and assigns LevelVariantSetsAsset to it
    UFUNCTION(BlueprintCallable, Category = "VariantManager")
	static ALevelVariantSetsActor* CreateLevelVariantSetsActor(ULevelVariantSets* LevelVariantSetsAsset);

	// Returns a property path for all the properties we can capture for an actor. Will also
	// handle receiving the actor's class instead.
	UFUNCTION(BlueprintCallable, Category = "VariantManager")
	static TArray<FString> GetCapturableProperties(UObject* ActorOrClass);




	// Adds VariantSet to the LevelVariantSets' list of VariantSets
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void AddVariantSet(ULevelVariantSets* LevelVariantSets, UVariantSet* VariantSet);

	// Adds Variant to the VariantSet's list of Variants
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void AddVariant(UVariantSet* VariantSet, UVariant* Variant);

	// Binds the Actor to the Variant, internally creating a VariantObjectBinding
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void AddActorBinding(UVariant* Variant, AActor* Actor);

	// Finds the actor binding to Actor within Variant and tries capturing a property with PropertyPath
	// Returns the captured UPropertyValue if succeeded or nullptr if it failed.
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static UPropertyValue* CaptureProperty(UVariant* Variant, AActor* Actor, FString PropertyPath);



	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category="VariantManager" )
	static int32 AddDependency(UVariant* Variant, UPARAM(ref) FVariantDependency& Dependency);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category="VariantManager")
	static void SetDependency(UVariant* Variant, int32 Index, UPARAM(ref) FVariantDependency& Dependency);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category="VariantManager")
	static void DeleteDependency(UVariant* Variant, int32 Index);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category="VariantManager")
	static TArray<FVariantDependency> GetDependencies(UVariant* Variant);


	// Get functions are the ones exposed directly on the VariantSet/Variant types

	// Returns which properties have been captured for this actor in Variant
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static TArray<UPropertyValue*> GetCapturedProperties(UVariant* Variant, AActor* Actor);




	// Removes VariantSet from LevelVariantSets, if that is its parent
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveVariantSet(ULevelVariantSets* LevelVariantSets, UVariantSet* VariantSet);

	// Removes Variant from VariantSet, if that is its parent
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveVariant(UVariantSet* VariantSet, UVariant* Variant);

	// Removes an actor binding to Actor from Variant, if it exists
    UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveActorBinding(UVariant* Variant, AActor* Actor);

	// Removes a property capture from an actor binding within Variant, if it exists
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveCapturedProperty(UVariant* Variant, AActor* Actor, UPropertyValue* Property);




	// Looks for a variant set with VariantSetName within LevelVariantSets and removes it, if it exists
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveVariantSetByName(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName);

	// Looks for a variant with VariantName within VariantSet and removes it, if it exists
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveVariantByName(UVariantSet* VariantSet, const FString& VariantName);

	// Looks for an actor binding to an actor with ActorLabel within Variant and removes it, if it exists
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveActorBindingByName(UVariant* Variant, const FString& ActorName);

	// Removes property capture with PropertyPath from Actor's binding within Variant, if it exists
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void RemoveCapturedPropertyByName(UVariant* Variant, AActor* Actor, FString PropertyPath);




	// Records new data for PropVal from the actor from which it was captured
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void Record(UPropertyValue* PropVal);

	// Applies the recorded data from PropVal to the actor from which it was captured
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static void Apply(UPropertyValue* PropVal);

	// This allows the scripting language to get the type of the C++ property its dealing with
	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager")
	static FString GetPropertyTypeString(UPropertyValue* PropVal);



	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueBool(UPropertyValue* Property, bool InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static bool GetValueBool(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueInt(UPropertyValue* Property, int32 InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static int32 GetValueInt(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueFloat(UPropertyValue* Property, float InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static float GetValueFloat(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueObject(UPropertyValue* Property, UObject* InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static UObject* GetValueObject(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueString(UPropertyValue* Property, const FString& InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FString GetValueString(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueRotator(UPropertyValue* Property, FRotator InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FRotator GetValueRotator(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueColor(UPropertyValue* Property, FColor InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FColor GetValueColor(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueLinearColor(UPropertyValue* Property, FLinearColor InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FLinearColor GetValueLinearColor(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueVector(UPropertyValue* Property, FVector InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FVector GetValueVector(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueQuat(UPropertyValue* Property, FQuat InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FQuat GetValueQuat(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueVector4(UPropertyValue* Property, FVector4 InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FVector4 GetValueVector4(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueVector2D(UPropertyValue* Property, FVector2D InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FVector2D GetValueVector2D(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static void SetValueIntPoint(UPropertyValue* Property, FIntPoint InValue);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|PropertyAccessors")
	static FIntPoint GetValueIntPoint(UPropertyValue* Property);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static UBlueprint* GetOrCreateDirectorBlueprint(UVariant* Variant);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static TArray<FName> GetFunctionCallerNames(UVariant* Variant, AActor* Actor);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static bool GetFunctionCallerArguments(UVariant* Variant, AActor* Actor, FName FunctionName, TMap<FName, FString>& Arguments);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static bool CreateFunctionCaller(UVariant* Variant, AActor* Actor, FName FunctionName, EVariantFunctionCallerSignature Signature);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static bool AddFunctionCaller(UVariant* Variant, AActor* Actor, FName FunctionName);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static bool UpdateFunctionCallerArguments(UVariant* Variant, AActor* Actor, FName FunctionName, const TMap<FName, FString>& Arguments);

	UFUNCTION(BlueprintCallable, meta=(ScriptMethod), Category = "VariantManager|FunctionCallers")
	static bool RemoveFunctionCaller(UVariant* Variant, AActor* Actor, FName FunctionName);

private:
	static bool CheckTheFunctionSignature(UVariant* Variant, AActor* BoundActor, FName FunctionName, EVariantFunctionCallerSignature Signature);

private:
	static TUniquePtr<FVariantManager> VariantManager;
};
