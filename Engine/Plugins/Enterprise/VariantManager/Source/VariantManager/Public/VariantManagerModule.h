// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()

class FVariantManager;
class ULevelVariantSets;

#define VARIANTMANAGERMODULE_MODULE_NAME TEXT("VariantManager")

class IVariantManagerModule : public IModuleInterface
{
public:
	static inline IVariantManagerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IVariantManagerModule>(VARIANTMANAGERMODULE_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(VARIANTMANAGERMODULE_MODULE_NAME);
	}

	virtual TSharedRef<FVariantManager> CreateVariantManager(ULevelVariantSets* InLevelVariantSets) = 0;
};

