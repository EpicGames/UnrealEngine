// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class VariantManager : ModuleRules
	{
		public VariantManager(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"PropertyPath",  // For how we handle captured properties
					"VariantManagerContent"  // Data classes are in here
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AppFramework", // For color pickers (for color and linear color properties)
					"AssetTools",
                    "BlueprintGraph", // For function director
					"DesktopPlatform",
					"EditorStyle", // For FontAwesome glyphs
					"EditorWidgets",
					"GraphEditor", // For DragDropOp, might be removed later
					"InputCore", // For ListView keyboard control
					"Projects", // So that we can use the IPluginManager, required for our custom style
					"PropertyEditor",  // For functions that create the property widgets
					"CinematicCamera",  // So we can check the CineCamera structs exist
					"ContentBrowser",
					"SceneOutliner",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets",
					"VariantManagerContentEditor",
					"WorkspaceMenuStructure",
				}
			);
        }
	}
}