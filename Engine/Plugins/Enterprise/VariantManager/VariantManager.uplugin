{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Variant Manager",
	"Description": "Manages scene actor variants",
	"Category" : "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules" :
	[
		{
			"Name" : "VariantManager",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "VariantManagerContent",
			"Enabled": true
		},
    ]
}